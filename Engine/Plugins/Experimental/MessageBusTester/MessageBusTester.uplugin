{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MessageBus Tester",
	"Description": "Plugin to test and monitor message bus reliability",
	"Category": "Messaging",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "MessageBusTesterApp" ],
	"Plugins": [
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "MessageBusTester",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"AllowlistPrograms": [ "MessageBusTesterApp" ]
		},
		{
			"Name": "MessageBusTesterEditor",
          	"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"AllowlistPrograms": [ "MessageBusTesterApp" ]
		}
	]
}
