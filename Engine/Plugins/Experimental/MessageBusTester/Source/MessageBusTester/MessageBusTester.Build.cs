// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MessageBusTester : ModuleRules
{
	public MessageBusTester(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		bUseUnity = false;
		PCHUsage = PCHUsageMode.NoPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "DeveloperSettings",
				"MessagingCommon",
				"UdpMessaging"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Messaging",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MessageLog",
				}
			);
		}
	}
}
