// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiscoveredTester.h"

#include "MessageBusTesterModule.h"
#include "IMessageBusTesterLogger.h"
#include "MessageBusTesterSettings.h"

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "Features/IModularFeatures.h"

#include "INetworkMessagingExtension.h"

namespace UE::DiscoveredTester::Private
{
	inline IMessageBusTesterModule& Get()
	{
		const FName ModuleName = TEXT("MessageBusTester");
		return FModuleManager::LoadModuleChecked<IMessageBusTesterModule>(ModuleName);
	}

	inline bool IsAvailable()
	{
		const FName ModuleName = TEXT("MessageBusTester");
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}
}


#define LOCTEXT_NAMESPACE "DiscoveredTester"

void FDiscoveredTester::UpdateConnectionState()
{
	if (ConnectionState != EDiscoveredTesterConnectionState::Lost)
	{
		if ((FPlatformTime::Seconds() - LastUnreliableKeepAliveMessageTime > GetDefault<UMessageBusTesterSettings>()->TimeoutInterval)
			&& (FPlatformTime::Seconds() - LastReliableKeepAliveMessageTime > GetDefault<UMessageBusTesterSettings>()->TimeoutInterval))
		{
			UE_LOGF(LogMessageBusTester, Log, "Tester lost, %ls", *Descriptor.FriendlyName.ToString());
			UE::DiscoveredTester::Private::Get().GetLogger().Log(Descriptor.FriendlyName, FString::Printf(TEXT("Tester lost connection for missed keepalive")), EMessageSeverity::Warning);
			ConnectionState = EDiscoveredTesterConnectionState::Lost;
		}
	}
	else if (ConnectionState == EDiscoveredTesterConnectionState::Lost)
	{
		if ((FPlatformTime::Seconds() - LastUnreliableKeepAliveMessageTime <= GetDefault<UMessageBusTesterSettings>()->TimeoutInterval)
			|| (FPlatformTime::Seconds() - LastReliableKeepAliveMessageTime <= GetDefault<UMessageBusTesterSettings>()->TimeoutInterval))
		{
			UE_LOGF(LogMessageBusTester, Log, "Tester back alive, %ls", *Descriptor.FriendlyName.ToString());
			UE::DiscoveredTester::Private::Get().GetLogger().Log(Descriptor.FriendlyName, FString::Printf(TEXT("Tester recovered")), EMessageSeverity::Info);
			ConnectionState = EDiscoveredTesterConnectionState::Connected;
		}
	}

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
	{
		INetworkMessagingExtension& Network = ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
		FGuid Id = Network.GetNodeIdFromAddress(Address);
		Statistics = Network.GetLatestNetworkStatistics(Id);
	}
}

void FDiscoveredTester::NotifyPayloadSent(const FTestPayloadMessage& PayloadMessage)
{
	SentPayloads.AddUnique({PayloadMessage});
}

void FDiscoveredTester::NotifyAckReceived(const FTestPayloadReceptionMessage& AckMessage)
{
	const double ReceivedTime = FPlatformTime::Seconds();
	const int32 PayloadId = AckMessage.PayloadId;
	const int32 SentPayloadIndex = SentPayloads.IndexOfByPredicate([PayloadId](const FPayloadInfo& Other) { return Other.PayloadId == PayloadId; });
	if (SentPayloads.IsValidIndex(SentPayloadIndex))
	{
		FPayloadInfo ReceivedPayload = SentPayloads[SentPayloadIndex];
		const double Elapsed = FMath::Max(ReceivedTime - ReceivedPayload.SentTime, UE_KINDA_SMALL_NUMBER);
		const double MegaByte = 1024.0f * 1024.0f;
		const double BandwithMegaBytes = static_cast<double>(ReceivedPayload.PayloadSize) / MegaByte / Elapsed;

		// Compute running average. 
		AverageMbPerSecond.PushValue(BandwithMegaBytes);
		
		UE::DiscoveredTester::Private::Get().GetLogger().Log(Descriptor.FriendlyName, FString::Printf(TEXT("Packet %d acknowledged after %0.5f (%0.1f MB/s) - Avg (%0.1f MB/s)"), ReceivedPayload.PayloadId, Elapsed, BandwithMegaBytes, AverageMbPerSecond.GetAverage()), EMessageSeverity::Info);

		ReceivedPayload.AcknowledgedTime = ReceivedTime;
		SentPayloads.RemoveAtSwap(SentPayloadIndex);
		AcknowledgedPayloads.AddUnique(MoveTemp(ReceivedPayload));

	}
	else
	{
		ensureAlways(false);
	}
}

void FDiscoveredTester::NotifyKeepAliveReceived(bool bIsReliable)
{
	const double ReceptionTime = FPlatformTime::Seconds();
	if (bIsReliable)
	{
		KeepAliveStatistics.LastReliableKeepAliveReceptionInterval = ReceptionTime - LastReliableKeepAliveMessageTime;
		KeepAliveStatistics.MinReliableKeepAliveInterval = FMath::Min(KeepAliveStatistics.MinReliableKeepAliveInterval, KeepAliveStatistics.LastReliableKeepAliveReceptionInterval);
		KeepAliveStatistics.MaxReliableKeepAliveInterval = FMath::Max(KeepAliveStatistics.MaxReliableKeepAliveInterval, KeepAliveStatistics.LastReliableKeepAliveReceptionInterval);
		AverageReliableKeepAliveIntervalWindow.PushValue(KeepAliveStatistics.LastReliableKeepAliveReceptionInterval);
		KeepAliveStatistics.AverageReliableKeepAliveInterval = AverageReliableKeepAliveIntervalWindow.GetAverage();
		LastReliableKeepAliveMessageTime = ReceptionTime;
		
		UE_LOGF(LogMessageBusTester, Verbose, "Received Reliable KeepAlive from '%ls' after %0.2f", *Descriptor.FriendlyName.ToString(), KeepAliveStatistics.LastReliableKeepAliveReceptionInterval);
	}
	else
	{
		KeepAliveStatistics.LastUnreliableKeepAliveReceptionInterval = ReceptionTime - LastUnreliableKeepAliveMessageTime;
		KeepAliveStatistics.MinUnreliableKeepAliveInterval = FMath::Min(KeepAliveStatistics.MinUnreliableKeepAliveInterval, KeepAliveStatistics.LastUnreliableKeepAliveReceptionInterval);
		KeepAliveStatistics.MaxUnreliableKeepAliveInterval = FMath::Max(KeepAliveStatistics.MaxUnreliableKeepAliveInterval, KeepAliveStatistics.LastUnreliableKeepAliveReceptionInterval);
		AverageUnreliableKeepAliveIntervalWindow.PushValue(KeepAliveStatistics.LastUnreliableKeepAliveReceptionInterval);
		KeepAliveStatistics.AverageUnreliableKeepAliveInterval = AverageUnreliableKeepAliveIntervalWindow.GetAverage();
		LastUnreliableKeepAliveMessageTime = ReceptionTime;

		UE_LOGF(LogMessageBusTester, Verbose, "Received Unreliable KeepAlive from '%ls' after %0.2f", *Descriptor.FriendlyName.ToString(), KeepAliveStatistics.LastUnreliableKeepAliveReceptionInterval);
	}
}

bool FDiscoveredTester::IsPendingPayloadAck(int32 PayloadId) const
{
	return  INDEX_NONE != SentPayloads.IndexOfByPredicate([PayloadId](const FPayloadInfo& Other) { return Other.PayloadId == PayloadId; });

}

#undef LOCTEXT_NAMESPACE
