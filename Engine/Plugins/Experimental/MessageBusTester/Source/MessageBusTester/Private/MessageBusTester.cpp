// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTester.h"

#include "Containers/Ticker.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/DateTime.h"

#include "MessageBusTesterModule.h"
#include "MessageBusTesterSettings.h"

#include "CoreGlobals.h"

#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/FileManager.h"

#include "Templates/Function.h"

#include "Algo/RemoveIf.h"

namespace MessageBusTesterUtils
{
	static constexpr int32 SupportedMessageBusTesterVersion = 1;
	static constexpr int32 KeepAliveIntervalMilliSeconds = 1000;
}

class FMessageBusTesterHeartBeat : public FRunnable
{
public:
	explicit FMessageBusTesterHeartBeat(TFunction<void()> InKeepAliveFunc)
		: bIsRunning(false)
		, bStopRequested(false)
		, KeepAliveFunc(MoveTemp(InKeepAliveFunc))
	{
		Thread.Reset(FRunnableThread::Create(this, TEXT("MessageBusTester Heart Beat")));
	}

	~FMessageBusTesterHeartBeat()
	{
		if (Thread)
		{
			Thread->Kill(true);
			Thread = nullptr;
		}

	}
			
	bool IsRunning() const
	{
		return Thread && bIsRunning;
	}

	//~ FRunnable Interface
	virtual bool Init() override
	{
		bIsRunning = true;
		return true;
	}

	/** 
	 * Every second send out a keep alive message that other testers can see.
	 */
	virtual uint32 Run() override
	{
		while (!bStopRequested)
		{
			KeepAliveFunc();

			FPlatformProcess::SleepNoStats(1.0f);
		}
		return 0;
	}

	/** Stop the running thread */
	virtual void Stop() override
	{
		bStopRequested = true;
	}

	/** Thread was told to exit by the FRunnable interface. */
	virtual void Exit() override
	{
		bIsRunning = false;
	}
	//~ FRunnable Interface.

	/** Atomics for state tracking and external access to the runnable. */
	std::atomic<bool> bIsRunning = false;
	std::atomic<bool> bStopRequested = false;

	TFunction<void()> KeepAliveFunc;
	TUniquePtr<FRunnableThread> Thread;
};

FMessageBusTester::FMessageBusTester() = default;
FMessageBusTester::~FMessageBusTester()
{
	FCoreDelegates::OnPreExit.RemoveAll(this);

	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}
	
	Stop();
}

void FMessageBusTester::SetState(EMessageBusTesterState InState)
{
	if (InState == State)
	{
		return;
	}

	State = InState;
}

bool FMessageBusTester::IsActive() const
{
	return GetState() == EMessageBusTesterState::Active;
}

EMessageBusTesterState FMessageBusTester::GetState() const
{
	return State;
}

void FMessageBusTester::AddTestPlanItem(FTestPlanItem NewItem)
{
	UE_LOGF(LogMessageBusTester, Display, "Adding test plan size %d with interval %f.", NewItem.NumBytes, NewItem.IntervalSeconds);
	TestPlan.TestPlanItems.Emplace(MoveTemp(NewItem));
	OnTestPlanChanged().Broadcast();
}

void FMessageBusTester::RemoveTestPlanItem(int32 Index)
{
	if (TestPlan.TestPlanItems.IsValidIndex(Index))
	{
		TestPlan.TestPlanItems.RemoveAt(Index);
		OnTestPlanChanged().Broadcast();
	}
}

bool FMessageBusTester::StartTest()
{
	if (GetState() == EMessageBusTesterState::Idle)
	{
		SetupTestPlan();

		//Notify other testers we are starting
		FTesterStartTestMessage* NewMsg = FMessageEndpoint::MakeMessage<FTesterStartTestMessage>();
		NewMsg->TestPlan = TestPlan;
		SendMessage(NewMsg, EMessageBusTesterMessageFlags::Reliable);

		return true;
	}

	return false;
}

void FMessageBusTester::TriggerAppExit()
{
	const double ShutdownWaitTime = 3.0;
	ShutdownTime = FPlatformTime::Seconds() + ShutdownWaitTime;
}

bool FMessageBusTester::StopTest(bool bShouldExitOnStop)
{
	if (GetState() == EMessageBusTesterState::Active)
	{
		//Notify other testers we are stopping the test
		FTesterStopTestMessage* NewMsg = FMessageEndpoint::MakeMessage<FTesterStopTestMessage>();
		NewMsg->bEngineShouldExit = bShouldExitOnStop; 
		SendMessage(NewMsg, EMessageBusTesterMessageFlags::Reliable);

		if (bShouldExitOnStop)
		{
			TriggerAppExit();
		}

		SetState(EMessageBusTesterState::Idle);
		return true;
	}

	return false;
}

void FMessageBusTester::Initialize()
{
	FCoreDelegates::OnPreExit.AddRaw(this, &FMessageBusTester::OnPreExit);

	Identifier = FGuid::NewGuid();

	CachedDiscoveryMessage.Descriptor = MessageBusTesterUtils::GetInstanceDescriptor();

	const TCHAR* CommandLine = FCommandLine::Get();

	// Parse test plans to run.
	FString TestPlans;
	FParse::Value(CommandLine, TEXT("-MESSAGE_BUS_TESTER_PLANS="), TestPlans, false);

	TArray<FString> CommandLinePlans;
	TestPlans.ParseIntoArray(CommandLinePlans, TEXT(","));
	for (const FString& Plan : CommandLinePlans)
	{
		TArray<FString> Tokens;
		if (Plan.ParseIntoArray(Tokens, TEXT(":"), false) == 2)
		{
			const int32 PlanSize = FCString::Atoi(*Tokens[0]);
			if (PlanSize > 0)
			{
				const float Interval = FCString::Atof(*Tokens[1]);
				if (Interval >= 0.0)
				{
					AddTestPlanItem({PlanSize, Interval});
				}
				else
				{
					UE_LOGF(LogMessageBusTester, Error, "With command line flag -MESSAGE_BUS_TESTER_PLANS, invalid interval specified -> %ls", *Tokens[1]);
				}
			}
			else
			{
				UE_LOGF(LogMessageBusTester, Error, "With command line flag -MESSAGE_BUS_TESTER_PLANS, invalid plan size specified -> %ls", *Tokens[0]);
			}
		}
		else
		{
			UE_LOGF(LogMessageBusTester, Error, "With command line flag -MESSAGE_BUS_TESTER_PLANS, invalid plan specified. Expected <size>:<interval> -> %ls", *Plan);
		}
	}

	FString RunDuration;
	if (FParse::Value(CommandLine, TEXT("-MESSAGE_BUS_TESTER_RUN_DURATION="), RunDuration))
	{
		const double InDuration = FCString::Atod(*RunDuration);
		if (InDuration > 0)
		{
			TestDuration = InDuration;
			UE_LOGF(LogMessageBusTester, Display, "Setting run duration to '%f'", TestDuration);
		}
	}

	FString ExportToCSV;
	if (FParse::Value(CommandLine, TEXT("-MESSAGE_BUS_TESTER_EXPORT_TO_CSV="), ExportToCSV))
	{
		ResultsToCSV = ExportToCSV;
	}

	// Only start the tester on initialization if the user has asked for it.
	if (FParse::Param(FCommandLine::Get(), TEXT("-MESSAGE_BUS_TESTER_START_ON_LAUNCH")))
	{
		bStartOnLaunch = true;
		Start();
	}
	
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FMessageBusTester::Tick), 0.0f);
}

template<typename TInboundType>
void FMessageBusTester::HandleMessage(const TInboundType& Inbound, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FInboundMessage InMsg(Inbound, Context);
	InboundMessages.Enqueue(MoveTemp(InMsg));
}

template<>
void FMessageBusTester::HandleMessage<FTesterKeepAliveMessage>(const FTesterKeepAliveMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FScopeLock Lock(&DiscoveredTesterCriticalSection);

	const FGuid& SenderIdentifier = Message.Identifier;
	if (TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>* FoundTesterPtr = DiscoveredTesters.FindByPredicate([SenderIdentifier](const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Other) { return Other->Identifier == SenderIdentifier; }))
	{
		TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& FoundTester = *FoundTesterPtr;
		if (FoundTester->ConnectionState != EDiscoveredTesterConnectionState::Lost)
		{
			FoundTester->NotifyKeepAliveReceived(Message.bReliablySent);
			FoundTester->State = Message.State;
		}
	}
}

template<>
void FMessageBusTester::HandleMessage<FTestPayloadMessage>(const FTestPayloadMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FTestPayloadReceptionMessage* Response = FMessageEndpoint::MakeMessage<FTestPayloadReceptionMessage>();
	Response->PayloadId = Message.PayloadId;
	Response->TestPlanIndex = Message.TestPlanIndex;
	Response->ReceivedPayloadSize = Message.Payload.Num();
	SendMessage(Response, {Context->GetSender()}, EMessageBusTesterMessageFlags::Reliable);
}

void FMessageBusTester::Start()
{
	UE_LOGF(LogMessageBusTester, Log, "Starting MessageBusTester '%ls'", *CachedDiscoveryMessage.Descriptor.FriendlyName.ToString());

	auto Handler = [this](const auto& T, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		HandleMessage(T, Context);
	};
	
	const FString MessageEndpointName = TEXT("MessageBusTester");
	TesterEndpoint = FMessageEndpoint::Builder(*MessageEndpointName)
		.ReceivingOnAnyThread()
		.Handling<FTesterDiscoveryMessage>(Handler)
		.Handling<FTesterKeepAliveMessage>(Handler)
		.Handling<FTesterStartTestMessage>(Handler)
		.Handling<FTesterStopTestMessage>(Handler)
		.Handling<FTestPayloadMessage>(Handler)
		.Handling<FTestPayloadReceptionMessage>(Handler);

	if (TesterEndpoint.IsValid())
	{
		TesterEndpoint->Subscribe<FTesterDiscoveryMessage>();

		auto KeepAlive = [this](){SendKeepAlive_AnyThread();};
		MessageBusHeartBeat = MakeUnique<FMessageBusTesterHeartBeat>(MoveTemp(KeepAlive));

		if (TestPlan.TestPlanItems.Num() == 0)
		{
			AddTestPlanItem({ 10 * 1024 * 1024, 0.0f});
		}
	}
}

void FMessageBusTester::SendKeepAlive_AnyThread()
{
	FTesterKeepAliveMessage* NewKeepAlive = FMessageEndpoint::MakeMessage<FTesterKeepAliveMessage>(KeepAliveNumber++);
	NewKeepAlive->State = GetState();
	NewKeepAlive->bReliablySent = true;
	SendMessage(NewKeepAlive, EMessageBusTesterMessageFlags::Reliable);
	
	NewKeepAlive = FMessageEndpoint::MakeMessage<FTesterKeepAliveMessage>(KeepAliveNumber++);
	NewKeepAlive->State = GetState();
	NewKeepAlive->bReliablySent = false;
	SendMessage(NewKeepAlive, EMessageBusTesterMessageFlags::None);
}

namespace UE::MessageBusTester::Private
{
FString GetCSVHeader()
{
	TStringBuilder<512> Builder;
	Builder.Append(TEXT("UTC,"));
	Builder.Append(TEXT("IPv4Address,"));
	Builder.Append(TEXT("Bytes Sent,"));
	Builder.Append(TEXT("Bytes Received,"));
	Builder.Append(TEXT("Bytes Lost,"));
	Builder.Append(TEXT("Percent Lost,"));
	Builder.Append(TEXT("Average MB/s"));
	return Builder.ToString();
}

void WriteTesterDataToCSV(const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Tester, TUniquePtr<FArchive>& CSVArchive)
{
	check(CSVArchive);

	const FMessageTransportStatistics& Stat = Tester->Statistics;

	TStringBuilder<512> Builder;
	Builder.Append(FDateTime::UtcNow().ToString());
	Builder.Append(TEXT(","));

	Builder.Append(Stat.IPv4AsString);
	Builder.Append(TEXT(","));

	Builder.Appendf(TEXT("%d"), Stat.TotalBytesSent);
	Builder.Append(TEXT(","));

	Builder.Appendf(TEXT("%d"), Stat.TotalBytesReceived);
	Builder.Append(TEXT(","));
	
	Builder.Appendf(TEXT("%d"), Stat.TotalBytesLost);
	Builder.Append(TEXT(","));

	double PercentLoss = 0;
	if (Stat.TotalBytesSent > 0)
	{
		PercentLoss = 100 * (static_cast<double>(Stat.TotalBytesLost) / static_cast<double>(Stat.TotalBytesSent)); 
	}
	Builder.Appendf(TEXT("%.3f"), PercentLoss);
	Builder.Append(TEXT(","));

	Builder.Appendf(TEXT("%.3f"), Tester->AverageMbPerSecond.GetAverage());

	FString CSVLine = Builder.ToString();
	CSVLine += LINE_TERMINATOR;

	FTCHARToUTF8 CSVLineUTF8(*CSVLine);
	CSVArchive->Serialize((UTF8CHAR*)CSVLineUTF8.Get(), CSVLineUTF8.Length() * sizeof(UTF8CHAR));
}
	
}
void FMessageBusTester::ExportResultsToCSV()
{
	if (ResultsToCSV.IsEmpty())
	{
		return;
	}

	FString FullPath = FPaths::ConvertRelativePathToFull(ResultsToCSV);
	if (FPaths::DirectoryExists(FullPath))
	{
		UE_LOGF(LogMessageBusTester, Warning, "Invalid CSV filename ('%ls') provided. Skipping export of CSV data.", *FullPath);
		return;
	}

	FString FullPathDir = FPaths::GetPath(FullPath);
	if (!IFileManager::Get().MakeDirectory(*FullPathDir, true))
	{
		UE_LOGF(LogMessageBusTester, Error, "Error creating folders to host CSV ('%ls').", *FullPath);
		return;
	}

	const bool bWriteHeader = !IFileManager::Get().FileExists(*FullPath);
	TUniquePtr<FArchive> CSVArchive = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FullPath, EFileWrite::FILEWRITE_Append | EFileWrite::FILEWRITE_AllowRead));

	if (!CSVArchive)
	{
		return;
	}

	if (bWriteHeader)
	{
		UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
		CSVArchive->Serialize(&UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR));
		
		FString CSVHeader = UE::MessageBusTester::Private::GetCSVHeader();
		CSVHeader += LINE_TERMINATOR;
		
		FTCHARToUTF8 CSVHeaderUTF8(*CSVHeader);
		CSVArchive->Serialize((UTF8CHAR*)CSVHeaderUTF8.Get(), CSVHeaderUTF8.Length() * sizeof(UTF8CHAR));
	}

	for (const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Tester : DiscoveredTesters)
	{
		UE::MessageBusTester::Private::WriteTesterDataToCSV(Tester, CSVArchive);
	}

	CSVArchive->Flush();	
}

void FMessageBusTester::Stop()
{
	StopTest();
	
	if (MessageBusHeartBeat.IsValid())
	{
		MessageBusHeartBeat->Stop();
		MessageBusHeartBeat.Reset();
	}

	//Let providers know we're out
	if (TesterEndpoint.IsValid())
	{
		// Disable the Endpoint message handling since the message could keep it alive a bit.
		TesterEndpoint->Disable();
		TesterEndpoint.Reset();


		// Now write the file results to a file.  It is possible for Stop to get called more than once soo
		// this ensures that we only write the result once the test has been completed.
		// 
		ExportResultsToCSV();
	}

	// Clear the inbound message queue.
	while (TOptional<FInboundMessage> Msg = InboundMessages.Dequeue()) {}

	InvalidTesters.Empty();
}

bool FMessageBusTester::Tick(float DeltaTime)
{
	UpdateDiscoveredTester();
	
	if (MessageBusHeartBeat.IsValid())
	{
		SendDiscoveryMessage();
	}

	// Clear the inbound message queue.
	while (TOptional<FInboundMessage> Msg = InboundMessages.Dequeue())
	{
		auto ProcessOne = [this, Context=Msg->Context](const auto& InboundMessage)
		{
			ProcessOnGameThread(InboundMessage, Context);
		};
		
		Visit(ProcessOne, Msg->Type);
	}

	if (bStartOnLaunch && GetState() != EMessageBusTesterState::Active)
	{
		FScopeLock Lock(&DiscoveredTesterCriticalSection);
		if (DiscoveredTesters.Num() > 0)
		{
			StartTest();
			// Reset our flag so we don't re-enter.
			bStartOnLaunch = false;
		}
	}

	if (GetState() == EMessageBusTesterState::Active)
	{
		UpdateTestSequence();

		if (StartTime > 0 && TestDuration > 0)
		{
			if (FPlatformTime::Seconds() > (StartTime + TestDuration))
			{
				UE_LOGF(LogMessageBusTester, Display, "Automatically stopping test because we have exceeded our test duration.");
				StopTest(true);
			}
		}
	}

	if (ShutdownTime > 0 && FPlatformTime::Seconds() > ShutdownTime)
	{
		RequestEngineExit(TEXT("Message bus tester shutdown requested"));		
	}

	return true;
}

void FMessageBusTester::UpdateDiscoveredTester()
{
	FScopeLock Lock(&DiscoveredTesterCriticalSection);

	for (TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Tester : DiscoveredTesters)
	{
		Tester->UpdateConnectionState();
	}
}

void FMessageBusTester::SendDiscoveryMessage()
{
	if (TesterEndpoint && (FPlatformTime::Seconds() - LastSentDiscoveryMessage) > GetDefault<UMessageBusTesterSettings>()->DiscoveryMessageInterval)
	{
		//Broadcast discovery signal
		FTesterDiscoveryMessage* DiscoveryMessage = FMessageEndpoint::MakeMessage<FTesterDiscoveryMessage>();
		DiscoveryMessage->Identifier = Identifier;
		DiscoveryMessage->Descriptor = CachedDiscoveryMessage.Descriptor;

		TesterEndpoint->Publish(DiscoveryMessage, EMessageScope::Network, FTimespan::Zero(), FDateTime::MaxValue());

		LastSentDiscoveryMessage = FPlatformTime::Seconds();
	}
}

void FMessageBusTester::OnPreExit()
{
	Stop();
}

void FMessageBusTester::UpdateTestSequence()
{
	FScopeLock Lock(&DiscoveredTesterCriticalSection);

	if (DiscoveredTesters.Num() <= 0)
	{
		return;
	}

	//Go through each discovered tester and send the next test plan payload if last was was received
	for (const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Tester : DiscoveredTesters)
	{
		if (Tester->ConnectionState != EDiscoveredTesterConnectionState::Lost
		&& Tester->State == EMessageBusTesterState::Active)
		{
			for (int32 Index = 0; Index < TestPlanItemTrackers.Num(); ++Index)
			{
				FTestPlanItemTracker& Test = TestPlanItemTrackers[Index];
				if ((FPlatformTime::Seconds() - Test.LastPayloadSentTimestamp) > Test.ItemDescription.IntervalSeconds)
				{
					bool bCanSendPayload = true;
					if (int32* SentPayloadId = Test.SentPayloadIds.Find(Tester->Identifier))
					{
						if (Tester->IsPendingPayloadAck(*SentPayloadId))
						{
							bCanSendPayload = false;
						}
					}

					if (bCanSendPayload)
					{
						static int32 NextPayloadId = 0;

						//Interval elapsed, resend test
						FTestPayloadMessage* NewPayload = FMessageEndpoint::MakeMessage<FTestPayloadMessage>();
						NewPayload->Payload.SetNum(Test.ItemDescription.NumBytes);
						NewPayload->TestPlanIndex = Index;

						const int32 PayloadId = ++NextPayloadId;
						NewPayload->PayloadId = PayloadId;
						Tester->NotifyPayloadSent(*NewPayload);

						SendMessage(NewPayload, {Tester->Address}, EMessageBusTesterMessageFlags::Reliable);

						//Timestamp starts after pushing the packet in the stack
						Test.LastPayloadSentTimestamp = FPlatformTime::Seconds();

						//Mark all pending reception to false
						Test.SentPayloadIds.FindOrAdd(Tester->Identifier) = PayloadId;

						UE_LOGF(LogMessageBusTester, Log, "Sending testplan item %d (payload %d of size %d)", Index, PayloadId, Test.ItemDescription.NumBytes);
					}
				}
			}
		}
	}
}

void FMessageBusTester::SetupTestPlan()
{
	TestPlanItemTrackers.Empty(TestPlan.TestPlanItems.Num());
	for (const FTestPlanItem& Item : TestPlan.TestPlanItems)
	{
		FTestPlanItemTracker NewTest;
		NewTest.ItemDescription = Item;
		TestPlanItemTrackers.Emplace(MoveTemp(NewTest));
	}

	SetState(EMessageBusTesterState::Active);
}

bool FMessageBusTester::SendMessageInternal(FTesterBaseMessage* Payload, UScriptStruct* Type, const TArray<FMessageAddress>& Addresses, EMessageBusTesterMessageFlags InFlags) const
{
	if (Addresses.Num() > 0 && TesterEndpoint.IsValid())
	{
		//Set identifier in sent message
		Payload->Identifier = Identifier;

		const EMessageFlags Flags = EnumHasAnyFlags(InFlags, EMessageBusTesterMessageFlags::Reliable) ? EMessageFlags::Reliable : EMessageFlags::None;
		TesterEndpoint->Send(Payload, Type, Flags, nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
		return true;
	}
	// We take ownership of Payload so we must release the memory here.  MessageBus uses FMalloc for allocation so we must call Free.
	FMemory::Free(Payload); 
	return false;
}

void FMessageBusTester::HandleDiscoveredTester(const FGuid& InIdentifier, const FTesterInstanceDescriptor& Descriptor, const FMessageAddress& Address)
{
	// -already known
	// -known in the passed but cleared
	// -unknown but matchable to previous instance

	FScopeLock Lock(&DiscoveredTesterCriticalSection);
	
	const auto FindTesterLambda = [InIdentifier, &Descriptor](const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Other)
	{
		//Look for a matching lost/disconnected provider from the same machine + same stage
		if (Other->Descriptor.MachineName == Descriptor.MachineName && Other->Descriptor.FriendlyName == Descriptor.FriendlyName)
		{
			return true;
		}

		return (Other->Identifier == InIdentifier);
	};

	const int32 ExistingProviderIndex = DiscoveredTesters.IndexOfByPredicate(FindTesterLambda);
	if (ExistingProviderIndex != INDEX_NONE)
	{
		//Need to better support rediscovering a dropped tester. Especially during an active test plan. 
		
		//In case of an existing provider, 
		TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Entry = DiscoveredTesters[ExistingProviderIndex];
		if (Entry->ConnectionState == EDiscoveredTesterConnectionState::Lost)
		{
			//Need to update data about a closed tester. Its address will certainly have changed and descriptor could also have
			const FGuid PreviousIdentifier = Entry->Identifier;
			FillTesterDescription(InIdentifier, Descriptor, Address, Entry);

			if (InIdentifier != PreviousIdentifier)
			{
				//If identifier change, make sure our mapping is up to date
				ensureAlways(false);
				//If we're reslotting an old provider to a new one (same stagename / same machine) trigger a list change to let listeners know about it
				OnDiscoveredTesterListChanged().Broadcast();
			}

			UE_LOGF(LogMessageBusTester, Log, "Old tester recovered, %ls - (%ls (%d)) on sessionId %d"
				, *Descriptor.FriendlyName.ToString()
				, *Descriptor.MachineName
				, Descriptor.ProcessId
				, Descriptor.SessionId);
		}
	}
	else
	{
		AddTester(InIdentifier, Descriptor, Address);
	}
}

bool FMessageBusTester::AddTester(const FGuid& InIdentifier, const FTesterInstanceDescriptor& Descriptor, const FMessageAddress& Address)
{
	FScopeLock Lock(&DiscoveredTesterCriticalSection);

	if (DiscoveredTesters.ContainsByPredicate([InIdentifier](const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Other) { return Other->Identifier == InIdentifier; }))
	{
		return false;
	}

	UE_LOGF(LogMessageBusTester, Log, "Adding tester %ls - (%ls (%d)) on sessionId %d"
		, *Descriptor.FriendlyName.ToString()
		, *Descriptor.MachineName
		, Descriptor.ProcessId
		, Descriptor.SessionId);
	
	TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& NewEntry = DiscoveredTesters.Emplace_GetRef(MakeShared<FDiscoveredTester, ESPMode::ThreadSafe>());
	FillTesterDescription(InIdentifier, Descriptor, Address, NewEntry);

	//Let know a new tester was added to the list
	OnDiscoveredTesterListChanged().Broadcast();

	return true;
}

void FMessageBusTester::FillTesterDescription(const FGuid& InIdentifier, const FTesterInstanceDescriptor& NewDescriptor, const FMessageAddress& NewAddress, TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& OutEntry) const
{
	OutEntry->Identifier = InIdentifier;
	OutEntry->Descriptor = NewDescriptor;
	OutEntry->Address = NewAddress;
	OutEntry->State = EMessageBusTesterState::Idle;
	OutEntry->ConnectionState = EDiscoveredTesterConnectionState::Connected;
	OutEntry->LastReceivedMessageTime = FPlatformTime::Seconds();
	OutEntry->LastReliableKeepAliveMessageTime = OutEntry->LastReceivedMessageTime;
	OutEntry->LastUnreliableKeepAliveMessageTime = OutEntry->LastReceivedMessageTime;
}

void FMessageBusTester::ProcessOnGameThread(const FTesterDiscoveryMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	//Don't listen to ourselves
	if (Message.Identifier == Identifier)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMessageBusTester::HandleDiscoveryMessage);

	if (Message.Descriptor.MessageVersion != MessageBusTesterUtils::SupportedMessageBusTesterVersion)
	{
		//Keep track of invalid providers to avoid log spamming
		if (!InvalidTesters.Contains(Message.Identifier))
		{
			InvalidTesters.Add(Message.Identifier);

			UE_LOGF(LogMessageBusTester, Warning, "Received discovery response from Tester '%ls' with unsupported version '%d'. Supported version is '%d'"
				, *Message.Descriptor.FriendlyName.ToString()
				, Message.Descriptor.MessageVersion
				, MessageBusTesterUtils::SupportedMessageBusTesterVersion);
		}

		return;
	}

	const UMessageBusTesterSettings* Settings = GetDefault<UMessageBusTesterSettings>();
	if (!Settings->bUseSessionId || Settings->GetSessionId() == Message.Descriptor.SessionId)
	{
		HandleDiscoveredTester(Message.Identifier, Message.Descriptor, Context->GetSender());
	}
	else if (!InvalidTesters.Contains(Message.Identifier))
	{
		//Keep track of invalid providers to avoid log spamming
		InvalidTesters.Add(Message.Identifier);

		UE_LOGF(LogMessageBusTester, Warning, "Received discovery response from Tester '%ls' with SessionId '%d' but expected '%d'"
			, *Message.Descriptor.FriendlyName.ToString()
			, Message.Descriptor.SessionId
			, Settings->GetSessionId());
	}
}

void FMessageBusTester::ProcessOnGameThread(const FTesterStartTestMessage& Message,	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	TestPlan = Message.TestPlan;
	OnTestPlanChanged().Broadcast();
	SetupTestPlan();
}

void FMessageBusTester::ProcessOnGameThread(const FTesterStopTestMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.bEngineShouldExit)
	{
		UE_LOGF(LogMessageBusTester, Display, "Remote connection requested shutdown.");

		TriggerAppExit();
	}

	SetState(EMessageBusTesterState::Idle);
}

void FMessageBusTester::ProcessOnGameThread(const FTestPayloadReceptionMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (TestPlanItemTrackers.IsValidIndex(Message.TestPlanIndex))
	{
		if (TestDuration > 0 && StartTime < 0)
		{
			// Only start our timer once we start receiving data.
			StartTime = FPlatformTime::Seconds();
		}

		const FGuid& SenderIdentifier = Message.Identifier;
		if (TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>* FoundTesterPtr = DiscoveredTesters.FindByPredicate([SenderIdentifier](const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Other) { return Other->Identifier == SenderIdentifier; }))
		{
			(*FoundTesterPtr)->NotifyAckReceived(Message);
		}
	}
}

bool FMessageBusTester::ClearLostTesters()
{
	FScopeLock Lock(&DiscoveredTesterCriticalSection);
	
	int32 OldSize = DiscoveredTesters.Num();
	int32 NewSize = Algo::RemoveIf(DiscoveredTesters, [](const TSharedPtr<FDiscoveredTester>& Tester) {
		return Tester.IsValid() && Tester->ConnectionState == EDiscoveredTesterConnectionState::Lost;
	});
	DiscoveredTesters.SetNum(NewSize);
	return OldSize != NewSize;
}
