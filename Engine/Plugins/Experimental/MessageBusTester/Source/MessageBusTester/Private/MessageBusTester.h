// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageBusTester.h"

#include "Containers/Ticker.h"

#include "DiscoveredTester.h"
#include "IMessageContext.h"
#include "UObject/ObjectMacros.h"

#include <atomic>
#include "Async/Future.h"

#include "Containers/MpscQueue.h"
#include "Misc/TVariant.h"
#include "Templates/UniquePtr.h"

class FMessageEndpoint;
class FMessageBusTesterHeartBeat;

using FMessageBusTesterTypes = TVariant<FTesterDiscoveryMessage, FTesterStartTestMessage, FTesterStopTestMessage, FTestPayloadReceptionMessage>;

/**
 * Implementation of the MessageBus Tester
 */
class FMessageBusTester : public IMessageBusTester
{
public:
	struct FInboundMessage
	{
		template<typename T>
		FInboundMessage(const T& InType, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
			: Context(InContext)
		{
			Type.Set<T>(InType);
		}

		FMessageBusTesterTypes Type;
		TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context;
	};
		
	FMessageBusTester();
	FMessageBusTester(const FMessageBusTester&) = delete;
	FMessageBusTester& operator=(const FMessageBusTester&) = delete;
	virtual ~FMessageBusTester();

	//~Begin IMessageBusTester interface
	virtual bool IsActive() const override;
	virtual FOnDiscoveredTesterListChanged& OnDiscoveredTesterListChanged() override { return DiscoveredTesterListChangedDelegate; }
	virtual FOnTestPlanChanged& OnTestPlanChanged() override { return TestPlanChangedDelegate; }
	virtual TConstArrayView<TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>> GetDiscoveredTesters() const override
	{
		return MakeArrayView(DiscoveredTesters);
	}

	virtual bool IsRunning() const override
	{
		return TesterEndpoint.IsValid();		
	}
	virtual bool StartSystem() override
	{
		Start();
		return true;
	}
	virtual bool StopSystem() override
	{
		Stop();
		return true;
	}
		
	virtual bool ClearLostTesters() override;		
	virtual const FMessageBusTestPlan& GetTestPlan() const override { return TestPlan; }
	virtual void AddTestPlanItem(FTestPlanItem NewItem) override;
	virtual void RemoveTestPlanItem(int32 Index) override;

	virtual bool StartTest() override;
	virtual bool StopTest(bool bShouldExitOnStop = false) override;
	virtual EMessageBusTesterState GetState() const override;
	//~End IMessageBusTester


	void Initialize();
	void Start();
	void Stop();

private:
	friend class FMessageBusTesterHeartBeat;

	void SendKeepAlive_AnyThread();
		
	/** Sends a message using messagebus to connected providers */
	bool SendMessageInternal(FTesterBaseMessage* Payload, UScriptStruct* Type, const TArray<FMessageAddress>& Addresses, EMessageBusTesterMessageFlags InFlags) const;


	void HandleDiscoveredTester(const FGuid& Identifier, const FTesterInstanceDescriptor& Descriptor, const FMessageAddress& Address);
	bool AddTester(const FGuid& Identifier, const FTesterInstanceDescriptor& Descriptor, const FMessageAddress& Address);
	void FillTesterDescription(const FGuid& Identifier, const FTesterInstanceDescriptor& NewDescriptor, const FMessageAddress& NewAddress, TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& OutEntry) const;
		
	void ProcessOnGameThread(const FTesterDiscoveryMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void ProcessOnGameThread(const FTesterStartTestMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void ProcessOnGameThread(const FTesterStopTestMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void ProcessOnGameThread(const FTestPayloadReceptionMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	
	/**  */
	bool Tick(float DeltaTime);

	/** Updates each provider's state based on last communication timestamp */
	void UpdateDiscoveredTester();

	/** Broadcasts a discovery message to find new providers and also keep communication active with all providers */
	void SendDiscoveryMessage();

	/** Shutdowns monitor before everything has exited. Let know providers that we're out */
	void OnPreExit();

	void UpdateTestSequence();

	/** Generic send message to support constructor parameters directly and temp object created */
	template<typename MessageType>
	bool SendMessage(MessageType* Payload, EMessageBusTesterMessageFlags Flags)
	{
		static_assert(TIsDerivedFrom<MessageType, FTesterBaseMessage>::IsDerived, "MessageType must be a FTesterBaseMessage derived UStruct.");

		FScopeLock Lock(&DiscoveredTesterCriticalSection);

		TArray<FMessageAddress> TestersAddress;
		TestersAddress.Reserve(DiscoveredTesters.Num());
		for (const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Tester : DiscoveredTesters)
		{
			if (Tester->ConnectionState == EDiscoveredTesterConnectionState::Connected)
			{
				TestersAddress.Add(Tester->Address);
			}
		}
		return SendMessageInternal(Payload, MessageType::StaticStruct(), TestersAddress, Flags);
	}
	
	/** Generic send message to support constructor parameters directly and temp object created */
	template<typename MessageType>
	bool SendMessage(MessageType* Payload, const TArray<FMessageAddress>& Addresses, EMessageBusTesterMessageFlags Flags)
	{
		static_assert(TIsDerivedFrom<MessageType, FTesterBaseMessage>::IsDerived, "MessageType must be a FTesterBaseMessage derived UStruct.");
		return SendMessageInternal(Payload, MessageType::StaticStruct(), Addresses, Flags);
	}

	void SetupTestPlan();

	template<typename TInboundType>
	void HandleMessage(const TInboundType& Inbound, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	void SetState(EMessageBusTesterState InState);

private:
	/** Dump test results to a CSV file. */
	void ExportResultsToCSV();
		
	/** Set the shutdown timer to trigger this app to exit. */
	void TriggerAppExit();

	/** Messages received from message bus. */
	TMpscQueue<FInboundMessage> InboundMessages;

	/** Unique pointer to the heart beat thread runnable. */
	TUniquePtr<FMessageBusTesterHeartBeat> MessageBusHeartBeat;

	/** Endpoint used to communicate with other testers */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> TesterEndpoint;

	/** List of discovered endpoints that we are tracking for testing. */
	TArray<TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>> DiscoveredTesters;

	/** List of providers we received discovery response from that were not usable (wrong version, session id...) */
	TArray<FGuid> InvalidTesters;

	/** Current state of the tester. */
	std::atomic<EMessageBusTesterState> State = EMessageBusTesterState::Idle;

	/** Tracks the current keep alive counter. */
	std::atomic<uint64> KeepAliveNumber = 0;
		
	/** Timestamp when we last sent a discovery message */
	double LastSentDiscoveryMessage = 0.0;

	/** Handle to our ticking delegate. */
	FTSTicker::FDelegateHandle TickerHandle;

	/** This tester identifier */
	FGuid Identifier;

	/** DiscoveryMessage built once and broadcasted periodically */
	FTesterDiscoveryMessage CachedDiscoveryMessage;

	FCriticalSection DiscoveredTesterCriticalSection;

	FOnDiscoveredTesterListChanged DiscoveredTesterListChangedDelegate;
	FOnTestPlanChanged TestPlanChangedDelegate;

	FMessageBusTestPlan TestPlan;

	/** Test start time. If bStartOnLaunch is specified then this value will get set on first payload reception. */
	double StartTime = -1;

	/** Time to execute shutdown. If a test duration has been specified then a shutdown time will be set at the end of the duration. */
	double ShutdownTime = -1;
		
	/** Run time duration in seconds. A number less than or equal to 0 indicates that we run forever.  If this is set we will also force the app to terminate. */
	double TestDuration = -1;

	/** Optional path to a CSV file that is written on exit. */
	FString ResultsToCSV;

	/** Indicates if we should execute our test plans as soon as we detect other clients. */
	bool bStartOnLaunch = false;
	
	struct FTestPlanItemTracker
	{
		FTestPlanItem ItemDescription;
		TMap<FGuid, int32> SentPayloadIds;
		double LastPayloadSentTimestamp = 0.0;
	};
	TArray<FTestPlanItemTracker> TestPlanItemTrackers;
};
