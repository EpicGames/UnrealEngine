// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTesterCommon.h"

#include "Misc/App.h"
#include "MessageBusTesterSettings.h"


namespace MessageBusTesterUtils
{
	static const FString CachedComputerName = FPlatformProcess::ComputerName();
	static const uint32 CachedProcessId = FPlatformProcess::GetCurrentProcessId();

	FTesterInstanceDescriptor GetInstanceDescriptor()
	{
		FTesterInstanceDescriptor Descriptor;

		//A machine could spawn multiple UE instances. Need to be able to differentiate them. ProcessId is there for that reason
		Descriptor.MachineName = CachedComputerName;
		Descriptor.ProcessId = CachedProcessId;

		Descriptor.FriendlyName = GetDefault<UMessageBusTesterSettings>()->CommandLineFriendlyName;
		if (Descriptor.FriendlyName == NAME_None)
		{
			const FString TempName = FString::Printf(TEXT("%s:%u"), *Descriptor.MachineName, Descriptor.ProcessId);
			Descriptor.FriendlyName = *TempName;
		}

		Descriptor.SessionId = GetDefault<UMessageBusTesterSettings>()->GetSessionId();

		return Descriptor;
	}
}


