// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTesterLogger.h"

#include "MessageBusTesterModule.h"

void FMessageBusTesterLogger::Log(const FName& TesterName, FString&& InLogMessage, EMessageSeverity::Type Verbosity /* = EMessageSeverity::Info */)
{
	if (Verbosity == EMessageSeverity::Info)
	{
		UE_LOGF(LogMessageBusTester, Display, "%ls", *InLogMessage);
	}
	else if (Verbosity == EMessageSeverity::Warning)
	{
		UE_LOGF(LogMessageBusTester, Warning, "%ls", *InLogMessage);
	}
	else if (Verbosity == EMessageSeverity::Error)
	{
		UE_LOGF(LogMessageBusTester, Error, "%ls", *InLogMessage);
	}

	TSharedPtr<FMessageBusTesterLogEntry> NewEntry = MakeShared<FMessageBusTesterLogEntry>(TesterName, MoveTemp(InLogMessage), Verbosity);
	Entries.Add(NewEntry);
	OnMessageBusTesterNewLogReceived().Broadcast(NewEntry.ToSharedRef());
}

void FMessageBusTesterLogger::ClearLog()
{
	Entries.Empty();
	OnMessageBusTesterLogCleared().Broadcast();
}
