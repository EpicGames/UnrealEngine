// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageBusTesterLogger.h"



/**
 *  
 */
class FMessageBusTesterLogger : public IMessageBusTesterLogger
{
public:


	//~Begin IMessageBusTesterLogger interface
	virtual void Log(const FName& TesterName, FString&& InLogMessage, EMessageSeverity::Type Verbosity /* = EMessageSeverity::Info */) override;
	virtual void ClearLog() override;
	virtual FOnMessageBusTesterNewLogReceived& OnMessageBusTesterNewLogReceived() override { return NewLogReceivedDelegate; }
	virtual FOnMessageBusTesterLogCleared& OnMessageBusTesterLogCleared() override { return LogClearedDelegate; }
	//~End IMessageBusTesterLogger


private:

	TArray<TSharedPtr<FMessageBusTesterLogEntry>> Entries;

	FOnMessageBusTesterNewLogReceived NewLogReceivedDelegate;
	FOnMessageBusTesterLogCleared LogClearedDelegate;
};
