// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTesterModule.h"

#include "HAL/IConsoleManager.h"
#include "IMessageBusTesterLogger.h"
#include "Misc/CoreDelegates.h"
#include "MessageBusTester.h"
#include "MessageBusTesterLogger.h"

DEFINE_LOG_CATEGORY(LogMessageBusTester)

void FMessageBusTesterModule::StartupModule()
{
	MessageBusTester = MakeUnique<FMessageBusTester>();
	MessageBusTester->Initialize();

	Logger = MakeUnique<FMessageBusTesterLogger>();
}

void FMessageBusTesterModule::ShutdownModule()
{
	MessageBusTester.Reset();
}

IMessageBusTester& FMessageBusTesterModule::GetMessageBusTester()
{
	check(MessageBusTester.IsValid());
	return *MessageBusTester;
}

IMessageBusTesterLogger& FMessageBusTesterModule::GetLogger()
{
	check(Logger.IsValid());
	return *Logger;
}

IMPLEMENT_MODULE(FMessageBusTesterModule, MessageBusTester)



