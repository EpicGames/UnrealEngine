// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageBusTesterModule.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Templates/UniquePtr.h"

class FAutoConsoleCommand;
class FMessageBusTester;
class FMessageBusTesterLogger;

DECLARE_LOG_CATEGORY_EXTERN(LogMessageBusTester, Log, All);


class FMessageBusTesterModule : public IMessageBusTesterModule
{
public:	
	virtual ~FMessageBusTesterModule() = default;
	
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~End IModuleInterface

	//~Begin IMessageBusTesterModule interface
	 virtual IMessageBusTester& GetMessageBusTester() override;
	 virtual IMessageBusTesterLogger& GetLogger() override;
	//~End IMessageBusTesterModule interface

private:


protected:
	
	/** Single instance of the MessageBusTester */
	TUniquePtr<FMessageBusTester> MessageBusTester;
	TUniquePtr<FMessageBusTesterLogger> Logger;
};
