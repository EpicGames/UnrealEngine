// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTesterSettings.h"

#include "Misc/CommandLine.h"

UMessageBusTesterSettings::UMessageBusTesterSettings()
{
	int32 CommandLineSessionIdTmp = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-MessageBusTesterSessionId="), CommandLineSessionIdTmp))
	{
		CommandLineSessionId = CommandLineSessionIdTmp;
	}

	FParse::Value(FCommandLine::Get(), TEXT("-MessageBusTesterFriendlyName="), CommandLineFriendlyName);
}

FName UMessageBusTesterSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UMessageBusTesterSettings::GetSectionText() const
{
	return NSLOCTEXT("MessageBusTesteringPlugin", "MessageBusTesterSettingsSection", "MessageBus Tester");
}
#endif


int32 UMessageBusTesterSettings::GetSessionId() const
{
	if (CommandLineSessionId.IsSet())
	{
		return CommandLineSessionId.GetValue();
	}

	return SessionId;
}

