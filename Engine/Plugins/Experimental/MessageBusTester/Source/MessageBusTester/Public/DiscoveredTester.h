// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Misc/QualifiedFrameTime.h"
#include "UObject/ObjectMacros.h"
#include "IMessageContext.h"
#include "MessageBusTesterCommon.h"
#include "Math/MovingWindowAverageFast.h"
#include "INetworkMessagingExtension.h"

#define UE_API MESSAGEBUSTESTER_API

struct FPayloadInfo
{
	FPayloadInfo(const FTestPayloadMessage& PayloadMessage)
		: PayloadId(PayloadMessage.PayloadId)
		, SentTime(FPlatformTime::Seconds())
		, PayloadSize(PayloadMessage.Payload.Num())
	{
		
	}

	bool operator==(const FPayloadInfo& Other) const
	{
		return PayloadId == Other.PayloadId && PayloadSize == Other.PayloadSize;
	}
	
	int32 PayloadId = INDEX_NONE;
	double SentTime = 0.0;
	int64 PayloadSize = 0;
	double AcknowledgedTime = -1.0;
};

struct FKeepAliveStatistics
{
	double LastUnreliableKeepAliveReceptionInterval = 0.0;
	double MinUnreliableKeepAliveInterval = DBL_MAX;
	double MaxUnreliableKeepAliveInterval = 0.0;
	double AverageUnreliableKeepAliveInterval = 0.0;
	double LastReliableKeepAliveReceptionInterval = 0.0;
	double MinReliableKeepAliveInterval = DBL_MAX;
	double MaxReliableKeepAliveInterval = 0.0;
	double AverageReliableKeepAliveInterval = 0.0;
};

/**
*/
class  FDiscoveredTester
{
	bool operator==(const FDiscoveredTester& Other) const { return Identifier == Other.Identifier; }
	bool operator!=(const FDiscoveredTester& Other) const { return !(*this == Other); }

public:
	UE_API void UpdateConnectionState();
	UE_API void NotifyPayloadSent(const FTestPayloadMessage& PayloadMessage);
	UE_API void NotifyAckReceived(const FTestPayloadReceptionMessage& AckMessage);
	UE_API void NotifyKeepAliveReceived(bool bIsReliable);
	UE_API bool IsPendingPayloadAck(int32 PayloadId) const;

public:

	/** Identifier of this tester */
	FGuid Identifier;

	/** Detailed descriptor */
	FTesterInstanceDescriptor Descriptor;

	/** Address of this Tester */
	FMessageAddress Address;

	/** State of this Tester based on message reception */
	EMessageBusTesterState State = EMessageBusTesterState::Idle;

	/** State of this Tester based on message reception */
	EDiscoveredTesterConnectionState ConnectionState = EDiscoveredTesterConnectionState::Connected;

	FMessageTransportStatistics Statistics;

	/** Timestamp when last message was received based on FPlatformTime::Seconds */
	double LastReceivedMessageTime = 0.0;

	/** Timestamp when last KeepAlive was received based on FPlatformTime::Seconds */
	double LastUnreliableKeepAliveMessageTime = 0.0;

	/** Timestamp when last KeepAlive was received based on FPlatformTime::Seconds */
	double LastReliableKeepAliveMessageTime = 0.0;

	/** Last interval between keep alive reception */
	FMovingWindowAverageFast<double, 128> AverageUnreliableKeepAliveIntervalWindow;
	FMovingWindowAverageFast<double, 128> AverageReliableKeepAliveIntervalWindow;

	/** Running average of transfer performance. */
	FMovingWindowAverageFast<double, 128> AverageMbPerSecond;
		
	FKeepAliveStatistics KeepAliveStatistics;

	TArray<FPayloadInfo> SentPayloads;
	TArray<FPayloadInfo> AcknowledgedPayloads;
};

#undef UE_API
