// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MessageBusTesterCommon.h"

class FDiscoveredTester;

/**
 * Interface for the MessageBusTester */
class IMessageBusTester
{
public:

	virtual ~IMessageBusTester() {}
	
	/**
	 * Returns true if tester is active
	 */
	virtual bool IsActive() const = 0;

	virtual TConstArrayView<TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>> GetDiscoveredTesters() const = 0;

	/** Clear any lost testers and return true if any were removed. */
	virtual bool ClearLostTesters() = 0;

	virtual const FMessageBusTestPlan& GetTestPlan() const = 0;

	virtual bool IsRunning() const = 0;
	virtual bool StartSystem() = 0;
	virtual bool StopSystem() = 0;

	virtual bool StartTest() = 0;
	virtual bool StopTest(bool bShouldExitOnStop = false) = 0;
	virtual void AddTestPlanItem(FTestPlanItem NewItem) = 0;
	virtual void RemoveTestPlanItem(int32 Index) = 0;
	virtual EMessageBusTesterState GetState() const = 0;

	DECLARE_MULTICAST_DELEGATE(FOnDiscoveredTesterListChanged);
	virtual FOnDiscoveredTesterListChanged& OnDiscoveredTesterListChanged() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnTestPlanChanged);
	virtual FOnTestPlanChanged& OnTestPlanChanged() = 0;
};
