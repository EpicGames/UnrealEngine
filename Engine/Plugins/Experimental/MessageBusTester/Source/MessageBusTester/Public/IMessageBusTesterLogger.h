// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Logging/TokenizedMessage.h"


struct FMessageBusTesterLogEntry
{
	FMessageBusTesterLogEntry() = default;
	FMessageBusTesterLogEntry(FName InLogSource, FString&& InLogMessage, EMessageSeverity::Type InMessageSeverity)
		: Source(InLogSource)
		, LogMessage(MoveTemp(InLogMessage))
		, MessageSeverity(InMessageSeverity)
	{}

	FName Source = NAME_None;
	FString LogMessage;
	EMessageSeverity::Type MessageSeverity = EMessageSeverity::Info;
};


class IMessageBusTesterLogger
{
public:

	virtual ~IMessageBusTesterLogger() = default;

	virtual void Log(const FName& TesterName, FString&& InLogMessage, EMessageSeverity::Type Verbosity = EMessageSeverity::Info) = 0;
	virtual void ClearLog() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMessageBusTesterNewLogReceived, TSharedRef<FMessageBusTesterLogEntry> /*LogData*/);
	virtual FOnMessageBusTesterNewLogReceived& OnMessageBusTesterNewLogReceived() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnMessageBusTesterLogCleared);
	virtual FOnMessageBusTesterLogCleared& OnMessageBusTesterLogCleared() = 0;
};
