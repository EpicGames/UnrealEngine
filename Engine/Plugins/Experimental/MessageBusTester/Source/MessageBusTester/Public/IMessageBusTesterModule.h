// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IMessageBusTester;
class IMessageBusTesterLogger;

class  IMessageBusTesterModule : public IModuleInterface
{
public:	
	virtual IMessageBusTester& GetMessageBusTester() = 0;

	virtual IMessageBusTesterLogger& GetLogger() = 0;
};

