// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Misc/QualifiedFrameTime.h"
#include "UObject/ObjectMacros.h"
#include "IMessageContext.h"

#include "MessageBusTesterCommon.generated.h"

#define UE_API MESSAGEBUSTESTER_API

/** Message flags configuring how a message is sent */
UENUM()
enum class EMessageBusTesterMessageFlags : uint8
{
	None = 0,

	/** Sends this message as reliable, to make sure it's received by the receivers */
	Reliable = 1 << 0,
};
ENUM_CLASS_FLAGS(EMessageBusTesterMessageFlags);

/** States that a tester can be in */
UENUM()
enum class EMessageBusTesterState : uint8
{
	/** Testing is active (messages are sent between each testers) */
	Active,
	/** Tester is idle */
	Idle,
};

/** Connection state for a MessageBus tester. */
UENUM()
enum class EDiscoveredTesterConnectionState : uint8
{
	/** Tester is considered connected */
	Connected,
	/** Tester was considered lost */
	Lost,
};

/**
 * Holds descriptive information about another tester. 
 * Information that won't change for a session
 * Used when testers connects
 */
USTRUCT(MinimalAPI)
struct FTesterInstanceDescriptor
{
	GENERATED_BODY()

public:

	/** Machine name read from FPlatformProcess::ComputerName() */
	UPROPERTY()
	FString MachineName;

	/** ProcessId read from FPlatformProcess::GetCurrentProcessId */
	UPROPERTY()
	uint32 ProcessId = 0;

	/** Friendly name for this Unreal instance. If empty, this will be MachineName - ProcessId */
	UPROPERTY()
	FName FriendlyName;

	/** Session Id that may be used to differentiate different sessions on the network */
	UPROPERTY()
	int32 SessionId = INDEX_NONE;
	
	/** Provision for versioning a tester if we need to differentiate version. */
	UPROPERTY()
	int32 MessageVersion = 1;
};

/**
 *  Base structure for all message bus tester messages
 */
USTRUCT(MinimalAPI)
struct FTesterBaseMessage
{
	GENERATED_BODY()

public:

	/** Identifier of this instance */
	UPROPERTY()
	FGuid Identifier;

	/** FrameTime of the sender. */
	UPROPERTY()
	FQualifiedFrameTime FrameTime;
};

/**
 *  Message broadcasted periodically by the monitor to discover new providers 
 */
USTRUCT(MinimalAPI)
struct FTesterDiscoveryMessage : public FTesterBaseMessage
{
	GENERATED_BODY()

public:
	FTesterDiscoveryMessage() = default;
	FTesterDiscoveryMessage(FTesterInstanceDescriptor&& InDescriptor)
		: Descriptor(MoveTemp(InDescriptor))
	{}

public:

	/** Detailed description of a tester */
	UPROPERTY()
	FTesterInstanceDescriptor Descriptor;
};

/**
 * 
 */
USTRUCT(MinimalAPI)
struct FTesterKeepAliveMessage : public FTesterBaseMessage
{
	GENERATED_BODY()

public:
	FTesterKeepAliveMessage() = default;
	FTesterKeepAliveMessage(uint64 InNumber)
		: KeepAliveNumber(InNumber)
	{}


public:

	UPROPERTY()
	uint64 KeepAliveNumber = 0;

	UPROPERTY()
	bool bReliablySent = true;

	UPROPERTY()
	EMessageBusTesterState State = EMessageBusTesterState::Idle;
};

USTRUCT(MinimalAPI)
struct FTestPlanItem
{
	GENERATED_BODY()

	UPROPERTY()
	int32 NumBytes = 1024;

	UPROPERTY()
	float IntervalSeconds = 0;
};

USTRUCT(MinimalAPI)
struct FMessageBusTestPlan
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FTestPlanItem> TestPlanItems;
};

/**
 *
 */
USTRUCT(MinimalAPI)
struct FTesterStartTestMessage : public FTesterBaseMessage
{
	GENERATED_BODY()

public:
	FTesterStartTestMessage() = default;


public:
	UPROPERTY()
	FMessageBusTestPlan TestPlan;
};

/**
 *
 */
USTRUCT(MinimalAPI)
struct FTesterStopTestMessage : public FTesterBaseMessage
{
	GENERATED_BODY()

public:
	FTesterStopTestMessage() = default;

	UPROPERTY()
	bool bEngineShouldExit = false;
};



/**
 *
 */
USTRUCT(MinimalAPI)
struct FTestPayloadMessage : public FTesterBaseMessage
{
	GENERATED_BODY()

public:
	FTestPayloadMessage() = default;

public:
	
	UPROPERTY()
	int32 TestPlanIndex = INDEX_NONE;

	UPROPERTY()
	int32 PayloadId = 0;

	UPROPERTY()
	TArray<uint8> Payload;
};

/**
 *
 */
USTRUCT(MinimalAPI)
struct FTestPayloadReceptionMessage : public FTesterBaseMessage
{
	GENERATED_BODY()

public:
	FTestPayloadReceptionMessage() = default;

public:

	UPROPERTY()
	int32 TestPlanIndex = INDEX_NONE;

	UPROPERTY()
	int32 PayloadId = 0;

	UPROPERTY()
	int32 ReceivedPayloadSize = 0;
};

namespace MessageBusTesterUtils
{
	UE_API FTesterInstanceDescriptor GetInstanceDescriptor();
}

#undef UE_API
