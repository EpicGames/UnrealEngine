// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "UObject/Object.h"

#include "MessageBusTesterSettings.generated.h"

#define UE_API MESSAGEBUSTESTER_API 
/**
 * Settings for the MessageBusTester plugin modules. 
 */
UCLASS(config=Engine, MinimalAPI)
class UMessageBusTesterSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UMessageBusTesterSettings();
	//~ Begin UDevelopperSettings interface
	UE_API virtual FName GetCategoryName() const;
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
#endif
	//~ End UDevelopperSettings interface

	/** Returns current SessionId either based on settings or overriden by commandline */
	UE_API int32 GetSessionId() const;

public:

	/** If true, message bus will only listen to Testers with same sessionId */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bUseSessionId = true;

protected:
	/**
	 * The projects SessionId to differentiate data sent over network.
	 * @note It may be overriden via the command line, "-MessageBusTesterSessionId=1"
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	int32 SessionId = INDEX_NONE;

public:

	/** Interval threshold between KeepAlive reception before considering another tester as dropped */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta=(Units="s", ClampMin=0.5f))
	float TimeoutInterval = 10.0f;

	/** Interval between each discovery signal */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta=(Units="s", ClampMin=0.5f))
	float DiscoveryMessageInterval = 2.0f;

	/**
	 * The current SessionId in a MessageBusTesting context read from the command line.
	 * ie. "-MessageBusTesterSessionId=1"
	 */
	TOptional<int32> CommandLineSessionId;

	/**
	 * A friendly name for that instance given through command line (-MessageBusTesterFriendlyName=) to identify it when monitoring.
	 * If none, by default it will be filled with MachineName:ProcessId
	 */
	FName CommandLineFriendlyName;
};

#undef UE_API
