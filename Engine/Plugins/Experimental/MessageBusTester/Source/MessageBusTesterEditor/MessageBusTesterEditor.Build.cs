// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MessageBusTesterEditor : ModuleRules
{
	public MessageBusTesterEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"EditorStyle",
 				"InputCore",
 				"MessageLog",
				"Projects",
				"RenderCore",
				"Settings",
				"Slate",
				"SlateCore",
				"MessageBusTester",
                "WorkspaceMenuStructure",
				"UdpMessaging",
				"Messaging",
				"Networking"
            }
		);
	}
}
