// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTesterEditorModule.h"

#include "SMessageBusTesterPanel.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Docking/TabManager.h"

DEFINE_LOG_CATEGORY(LogMessageBusTesterEditor);

#define LOCTEXT_NAMESPACE "MessageBusTesterEditor"

void FMessageBusTesterEditorModule::DisplayMessageBusTester()
{
	FGlobalTabmanager::Get()->TryInvokeTab(SMessageBusTesterPanel::GetTabName());
}

void FMessageBusTesterEditorModule::StartupModule()
{
	SMessageBusTesterPanel::RegisterNomadTabSpawner(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsMiscCategory());
}

void FMessageBusTesterEditorModule::ShutdownModule()
{
	if (!IsRunningCommandlet() && UObjectInitialized() && !IsEngineExitRequested())
	{
		SMessageBusTesterPanel::UnregisterNomadTabSpawner();
	}
}


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMessageBusTesterEditorModule, MessageBusTesterEditor)
