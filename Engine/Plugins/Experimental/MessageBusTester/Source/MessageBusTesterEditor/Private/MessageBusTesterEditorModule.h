// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "Features/IModularFeatures.h"

#include "IMessageBusTesterEditorModule.h"
#include "INetworkMessagingExtension.h"

#include "IMessageBusTesterModule.h"

#include "Misc/ScopeExit.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMessageBusTesterEditor, Log, All);

namespace MessageBusTesterHelper
{
	inline IMessageBusTesterModule& Get()
	{
		const FName ModuleName = TEXT("MessageBusTester");
		return FModuleManager::LoadModuleChecked<IMessageBusTesterModule>(ModuleName);
	}

	inline bool IsAvailable()
	{
		const FName ModuleName = TEXT("MessageBusTester");
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}
}

class FMessageBusTesterEditorModule : public IMessageBusTesterEditorModule
{
public:
	virtual void DisplayMessageBusTester() override;

	
	static INetworkMessagingExtension* GetMessagingStatistics()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (IsInGameThread())
		{
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		else
		{
			ModularFeatures.LockModularFeatureList();
			ON_SCOPE_EXIT
			{
				ModularFeatures.UnlockModularFeatureList();
			};
		
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
		return nullptr;
	}

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~End IModuleInterface
};
