// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageBusTesterEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"


FMessageBusTesterEditorStyle::FMessageBusTesterEditorStyle()
	: FSlateStyleSet("MessageBusTesterEditorStyle")
{
	if (IPluginManager::Get().FindPlugin(TEXT("MessageBusTester")))
	{
		const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MessageBusTester"))->GetContentDir();
		SetContentRoot(ContentDir);
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	}

	// ListView
	{
		const FVector2D Icon8x8(8.0f, 8.0f);
		FTableRowStyle CriticalStateRow(FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"));
		CriticalStateRow.SetEvenRowBackgroundBrush(FSlateColorBrush(FLinearColor(0.2f, 0.0f, 0.0f, 0.1f)))
			.SetOddRowBackgroundBrush(FSlateColorBrush(FLinearColor(0.2f, 0.0f, 0.0f, 0.1f)));

		Set("TableView.CriticalStateRow", CriticalStateRow);
	}

	//Checkbox for live/preview mode
	{
		FCheckBoxStyle SwitchStyle = FCheckBoxStyle()
		.SetForegroundColor(FLinearColor::White)
		.SetUncheckedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetPadding(FMargin(0, 0, 0, 1));

		Set("ViewMode", SwitchStyle);
	}

	
	FSlateStyleRegistry::RegisterSlateStyle(*this);

#undef IMAGE_BRUSH
}

FMessageBusTesterEditorStyle::~FMessageBusTesterEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FMessageBusTesterEditorStyle& FMessageBusTesterEditorStyle::Get()
{
	static FMessageBusTesterEditorStyle Inst;
	return Inst;
}


