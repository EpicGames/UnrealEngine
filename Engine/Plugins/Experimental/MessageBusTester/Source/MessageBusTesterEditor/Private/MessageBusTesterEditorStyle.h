// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Slate style set that defines all the styles for the MessageBus Tester UI
 */
class FMessageBusTesterEditorStyle : public FSlateStyleSet
{
public:
	/** Access the singleton instance for this style set */
	static FMessageBusTesterEditorStyle& Get();


private:
	FMessageBusTesterEditorStyle();
	~FMessageBusTesterEditorStyle();
};
