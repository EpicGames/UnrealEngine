// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMessageBusTestLogger.h"

#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "MessageBusTesterEditorModule.h"
#include "IMessageBusTesterLogger.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SMessageBusTesterLogger"


namespace MessageBusTesterLoggerUtils
{
	const FName HeaderIdName_SourceTester = "Source";
	const FName HeaderIdName_LogMessage = "LogMessage";
}



/**
 * TableRow
 */
void SMessageBusTesterLogEntryTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
{
	LogEntry = InArgs._LogEntry;

	Super::FArguments Arg;

	Super::Construct(Arg, InOwerTableView);
}

TSharedRef<SWidget> SMessageBusTesterLogEntryTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (MessageBusTesterLoggerUtils::HeaderIdName_SourceTester == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterLogEntryTableRow::GetSourceTester)
			];
	}
	if (MessageBusTesterLoggerUtils::HeaderIdName_LogMessage == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterLogEntryTableRow::GetLogMessage)
				.ColorAndOpacity(this, &SMessageBusTesterLogEntryTableRow::GetLogMessageTextColor)
			];
	}

	return SNullWidget::NullWidget;
}

FText SMessageBusTesterLogEntryTableRow::GetSourceTester() const
{
	if (TSharedPtr<FMessageBusTesterLogEntry> EntryPtr = LogEntry.Pin())
	{
		return FText::FromName(EntryPtr->Source);
	}

	return FText::GetEmpty();
}

FText SMessageBusTesterLogEntryTableRow::GetLogMessage() const
{
	if (TSharedPtr<FMessageBusTesterLogEntry> EntryPtr = LogEntry.Pin())
	{
		return FText::FromString(EntryPtr->LogMessage);
	}

	return FText::GetEmpty();
}


FSlateColor SMessageBusTesterLogEntryTableRow::GetLogMessageTextColor() const
{
	if (TSharedPtr<FMessageBusTesterLogEntry> EntryPtr = LogEntry.Pin())
	{
		switch (EntryPtr->MessageSeverity) 
		{
			case EMessageSeverity::Warning: return FLinearColor::Yellow;
			case EMessageSeverity::Error: return FLinearColor::Red;
			case EMessageSeverity::PerformanceWarning: return FLinearColor::Yellow;
			case EMessageSeverity::Info: return FSlateColor::UseForeground();
			default: return FSlateColor::UseForeground();
		}
	}

	return FLinearColor::Gray;
}

SMessageBusTestLogger::~SMessageBusTestLogger()
{
	if (MessageBusTesterHelper::IsAvailable())
	{
		MessageBusTesterHelper::Get().GetLogger().OnMessageBusTesterNewLogReceived().RemoveAll(this);
		MessageBusTesterHelper::Get().GetLogger().OnMessageBusTesterLogCleared().RemoveAll(this);
	}
}

void SMessageBusTestLogger::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Fill)
		.Padding(10.f, 0.f, 10.f, 10.f)
		[
			SNew(SBorder)
			//.BorderImage(FEditorStyle::GetBrush("Menu.Background"))
			.Padding(2.f)
			[
				SAssignNew(LogEntriesListView, SListView<TSharedPtr<FMessageBusTesterLogEntry>>)
				.ListItemsSource(&LogEntries)
				.OnGenerateRow(this, &SMessageBusTestLogger::OnGenerateActivityRowWidget)
				.SelectionMode(ESelectionMode::None)
				//.AllowOverscroll(EAllowOverscroll::No)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(MessageBusTesterLoggerUtils::HeaderIdName_SourceTester)
					.FillWidth(25.f)
					.DefaultLabel(LOCTEXT("HeaderName_SourceName", "Tester Name"))
					+ SHeaderRow::Column(MessageBusTesterLoggerUtils::HeaderIdName_LogMessage)
					.FillWidth(75.f)
					.DefaultLabel(LOCTEXT("HeaderName_LogMessage", "Message"))
				)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ContentPadding(FMargin(4.f, 2.f))
				.OnClicked(this, &SMessageBusTestLogger::OnLogClearedClicked)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ClearLabel", "Clear"))
				]
			]
		]
	];

	MessageBusTesterHelper::Get().GetLogger().OnMessageBusTesterNewLogReceived().AddSP(this, &SMessageBusTestLogger::OnNewLogReceived);
	MessageBusTesterHelper::Get().GetLogger().OnMessageBusTesterLogCleared().AddSP(this, &SMessageBusTestLogger::OnLogCleared);
}

void SMessageBusTestLogger::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{

}

TSharedRef<ITableRow> SMessageBusTestLogger::OnGenerateActivityRowWidget(TSharedPtr<FMessageBusTesterLogEntry> InLogEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SMessageBusTesterLogEntryTableRow> Row = SNew(SMessageBusTesterLogEntryTableRow, OwnerTable)
		.LogEntry(InLogEntry);
	return Row;
}

void SMessageBusTestLogger::OnNewLogReceived(TSharedRef<FMessageBusTesterLogEntry> NewLog)
{
	LogEntries.Insert(NewLog, 0);
	LogEntriesListView->RequestListRefresh();
}

void SMessageBusTestLogger::OnLogCleared()
{
	LogEntries.Empty();
	LogEntriesListView->RequestListRefresh();
}

FReply SMessageBusTestLogger::OnLogClearedClicked()
{
	MessageBusTesterHelper::Get().GetLogger().ClearLog();
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
