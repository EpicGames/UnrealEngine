// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "IMessageBusTesterLogger.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SListView.h"

struct FMessageBusTesterLogEntry;

using FMessageBusTesterLogEntryPtr = TSharedPtr<FMessageBusTesterLogEntry>;

class SMessageBusTesterLogEntryTableRow : public SMultiColumnTableRow<FMessageBusTesterLogEntryPtr>
{
	using Super = SMultiColumnTableRow<FMessageBusTesterLogEntryPtr>;

public:
	SLATE_BEGIN_ARGS(SMessageBusTesterLogEntryTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<FMessageBusTesterLogEntry>, LogEntry)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTable);

private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FText GetSourceTester() const;
	FText GetLogMessage() const;
	FSlateColor GetLogMessageTextColor() const;

private:
	TWeakPtr<FMessageBusTesterLogEntry> LogEntry;
};


class SMessageBusTestLogger : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMessageBusTestLogger)
	{}
	SLATE_END_ARGS()

	virtual ~SMessageBusTestLogger();

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	//~ Begin SCompoundWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~End SCompoundWidget interface

private:
	TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FMessageBusTesterLogEntry> InLogEntry, const TSharedRef<STableViewBase>& OwnerTable);
	void OnNewLogReceived(TSharedRef<FMessageBusTesterLogEntry> NewLog);
	void OnLogCleared();
	FReply OnLogClearedClicked();

private:

	TSharedPtr<SListView<TSharedPtr<FMessageBusTesterLogEntry>>> LogEntriesListView;
	TArray<TSharedPtr<FMessageBusTesterLogEntry>> LogEntries;
};
