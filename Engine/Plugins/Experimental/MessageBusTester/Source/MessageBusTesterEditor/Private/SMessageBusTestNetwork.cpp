// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMessageBusTestNetwork.h"

#include "MessageBusTesterEditorModule.h"

#include "HAL/Platform.h"

#include "Algo/BinarySearch.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "INetworkMessagingExtension.h"

#include "SlateOptMacros.h"

#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SMessageBusTestNetwork"

namespace MessageBusTesterNetworkUtils
{
	const FName HeaderIdName_MessageId    = TEXT("MessageId");
	const FName HeaderIdName_SentSegments = TEXT("Sent");
	const FName HeaderIdName_AckSegments  = TEXT("SegmentsAck");
	const FName HeaderIdName_TotalSize    = TEXT("Size");
	const FName HeaderIdName_DataRate     = TEXT("DataRate");
}

void SMessageBusTesterSegmentersTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
{
	Segmenter = InArgs._Segmenter;

	Super::FArguments Arg;

	Super::Construct(Arg, InOwerTableView);
}

TSharedRef<SWidget> SMessageBusTesterSegmentersTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace MessageBusTesterNetworkUtils;
	if (HeaderIdName_MessageId == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterSegmentersTableRow::GetMessageId)
			];
	}
	if (HeaderIdName_SentSegments == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterSegmentersTableRow::GetSentSegments)
			];
	}
	if (HeaderIdName_AckSegments == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterSegmentersTableRow::GetAckSegments)
			];
	}
	if (HeaderIdName_TotalSize == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterSegmentersTableRow::GetSize)
			];
	}
	if (HeaderIdName_DataRate == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTesterSegmentersTableRow::GetDataRate)
			];
	}
	return SNullWidget::NullWidget;
}

FText SMessageBusTesterSegmentersTableRow::GetMessageId() const
{
	if(TSharedPtr<FSegmenterRowData> SegStats = Segmenter.Pin() )
	{
		return FText::AsNumber(SegStats->Stats.MessageId);
	}
	return FText::GetEmpty();
}

FText SMessageBusTesterSegmentersTableRow::GetSentSegments() const
{
	if(TSharedPtr<FSegmenterRowData> SegStats = Segmenter.Pin() )
	{
		return FText::AsNumber(SegStats->Stats.BytesSent);
	}
	return FText::GetEmpty();
}

FText SMessageBusTesterSegmentersTableRow::GetAckSegments() const
{
	if(TSharedPtr<FSegmenterRowData> SegStats = Segmenter.Pin() )
	{
		return FText::AsNumber(SegStats->Stats.BytesAcknowledged);
	}
	return FText::GetEmpty();
}

FText SMessageBusTesterSegmentersTableRow::GetSize() const
{
	if(TSharedPtr<FSegmenterRowData> SegStats = Segmenter.Pin() )
	{
		return FText::AsNumber(SegStats->Stats.BytesToSend);
	}
	return FText::GetEmpty();
}

FText SMessageBusTesterSegmentersTableRow::GetDataRate() const
{
	if(TSharedPtr<FSegmenterRowData> SegStats = Segmenter.Pin() )
	{
		FNumberFormattingOptions Options;
		Options.MaximumFractionalDigits = 2;
		Options.MinimumFractionalDigits = 2;
		return FText::Format(LOCTEXT("BitsPerSec", "{0}"), FText::AsNumber(SegStats->ComputeTransferRate(), &Options) );
	}
	return LOCTEXT("ZeroBits","0");
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMessageBusTestNetwork::Construct(const FArguments& InArgs)
{
	using namespace MessageBusTesterNetworkUtils;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SVerticalBox)
			// Toolbar
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestNetwork::GetIPAddress)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestNetwork::GetWindowSize)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestNetwork::GetAverageRTT)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
			SNew(STextBlock)
				.Text(this, &SMessageBusTestNetwork::GetSegmentsInFlight)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestNetwork::GetBytesSent)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestNetwork::GetSegmentsLost)
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SBorder)
			.Padding(2.f)
			[
				SAssignNew(SegmenterListView, SListView<TSharedPtr<FSegmenterRowData>>)
				.ListItemsSource(GetSegmenterListSource())
				.OnGenerateRow(this, &SMessageBusTestNetwork::OnGenerateActivityRowWidget)
				.SelectionMode(ESelectionMode::None)
				//.AllowOverscroll(EAllowOverscroll::No)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column( HeaderIdName_MessageId    )
					.FillWidth(2.f)
					.DefaultLabel(LOCTEXT("HeaderName_MessageId", "Id"))
					+ SHeaderRow::Column( HeaderIdName_SentSegments )
					.FillWidth(2.f)
					.DefaultLabel(LOCTEXT("HeaderName_SentSegments", "Sent"))
					+ SHeaderRow::Column( HeaderIdName_AckSegments  )
					.FillWidth(2.f)
					.DefaultLabel(LOCTEXT("HeaderName_AckSegments", "Ack"))
					+ SHeaderRow::Column( HeaderIdName_TotalSize    )
					.FillWidth(2.f)
					.DefaultLabel(LOCTEXT("HeaderName_TotalSize", "Size"))
					+ SHeaderRow::Column( HeaderIdName_DataRate     )
					.FillWidth(2.f)
					.DefaultLabel(LOCTEXT("HeaderName_Rate", "Rate (Mbit/s)"))
				)
			]
		]
	];

	if (INetworkMessagingExtension* Statistics = FMessageBusTesterEditorModule::GetMessagingStatistics())
	{
		Statistics->OnOutboundTransferUpdatedFromThread().AddRaw(this, &SMessageBusTestNetwork::HandleSegmenterUpdated);
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


SMessageBusTestNetwork::~SMessageBusTestNetwork()
{
	if (INetworkMessagingExtension* Statistics = FMessageBusTesterEditorModule::GetMessagingStatistics())
	{
		Statistics->OnOutboundTransferUpdatedFromThread().RemoveAll(this);
	}
}

void SMessageBusTestNetwork::HandleSegmenterUpdated(FOutboundTransferStatistics InStats)
{
	OutboundStats.Enqueue(MoveTemp(InStats));
}

const SMessageBusTestNetwork::FSegmenterStatsArray* SMessageBusTestNetwork::GetSegmenterListSource() const
{
	if (Item.IsValid())
	{
		TSharedPtr<FDiscoveredTesterTableRowData> Data = Item.Pin();
		if (Data.IsValid())
		{
			if (SMessageBusTestNetwork::FSegmenterStatsArray const * SegmenterData = AllSegmenters.Find(Data->EndpointIdentifier))
			{
				return SegmenterData;
			}	
		}
	}

	static FSegmenterStatsArray EmptyArray;
	return &EmptyArray;
}

TSharedRef<ITableRow> SMessageBusTestNetwork::OnGenerateActivityRowWidget(TSharedPtr<FSegmenterRowData> InSegmenter, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SMessageBusTesterSegmentersTableRow> Row = SNew(SMessageBusTesterSegmentersTableRow, OwnerTable)
		.Segmenter(InSegmenter);
	return Row;
}

void SMessageBusTestNetwork::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TArray<FOutboundTransferStatistics> StatsList;
	FOutboundTransferStatistics Stats;
	while (OutboundStats.Dequeue(Stats))
	{
		StatsList.Add(Stats);
	}
	
	if (StatsList.Num())
	{
		for (FOutboundTransferStatistics SingleStat : StatsList)
		{
			FSegmenterStatsArray& Array = AllSegmenters.FindOrAdd(SingleStat.DestinationId);
			TSharedPtr<FSegmenterRowData> SharedValue = MakeShared<FSegmenterRowData>(SingleStat);
			// We use upper bound because we want the array to be in reverse order.
			//
			auto Pos = Algo::UpperBound(Array, SharedValue, [](const TSharedPtr<FSegmenterRowData>& Value, const TSharedPtr<FSegmenterRowData>& Check)
			{
				if (Check.IsValid() && Value.IsValid())
				{
					return *Value < *Check;
				}
				return false;
			});
			if (Array.Num()>0 && Pos < Array.Num() && Array[Pos] && *Array[Pos] == SingleStat)
			{
				*Array[Pos] = SingleStat;
			}
			else
			{
				Array.Insert(MoveTemp(SharedValue),Pos);
			}
		}
		SegmenterListView->RequestListRefresh();
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


FMessageTransportStatistics SMessageBusTestNetwork::GetStats() const
{
	if (TSharedPtr<FDiscoveredTesterTableRowData> RowData = Item.Pin())
	{
		return RowData->LatestStats();
	}
	return {};
}

void SMessageBusTestNetwork::SetNetworkViewData(TWeakPtr<FDiscoveredTesterTableRowData> View)
{
	Item = View;
	if (SegmenterListView)
	{
		SegmenterListView->SetItemsSource(GetSegmenterListSource());
		SegmenterListView->RequestListRefresh();
	}
}

FText SMessageBusTestNetwork::GetIPAddress() const
{
	FMessageTransportStatistics Stats = GetStats();
	return FText::Format(LOCTEXT("CurrentIpAddress", "IPv4 Address {0}"), FText::FromString(Stats.IPv4AsString));
}

FText SMessageBusTestNetwork::GetAverageRTT() const
{
	FMessageTransportStatistics Stats = GetStats();
	return FText::Format(LOCTEXT("CurrentRTT", "Average RTT: {0} ms"), FText::AsNumber(Stats.AverageRTT.GetFractionMilli()));
}

FText SMessageBusTestNetwork::GetSegmentsInFlight() const
{
	FMessageTransportStatistics Stats = GetStats();
	return FText::Format(LOCTEXT("BytesInFlight", "Bytes Inflight: {0}"), FText::AsNumber(Stats.BytesInflight));
}

FText SMessageBusTestNetwork::GetWindowSize() const
{
	FMessageTransportStatistics Stats = GetStats();
	return FText::Format(LOCTEXT("WindowSize", "WindowSize : {0}"), FText::AsNumber(Stats.WindowSize));
}

FText SMessageBusTestNetwork::GetBytesSent() const
{
	FMessageTransportStatistics Stats = GetStats();
	return FText::Format(LOCTEXT("BytesSent", "Bytes Sent : {0}"), FText::AsNumber(Stats.TotalBytesSent));
}

FText SMessageBusTestNetwork::GetSegmentsLost() const
{
	FMessageTransportStatistics Stats = GetStats();
	double PercentLoss = 0;

	if (Stats.TotalBytesSent > 0)
	{
		PercentLoss = 100 * (static_cast<double>(Stats.TotalBytesLost) / static_cast<double>(Stats.TotalBytesSent)); 
	}

	FNumberFormattingOptions Options;
	Options.MaximumFractionalDigits = 2;
	Options.MinimumFractionalDigits = 2;
	return FText::Format(LOCTEXT("BytesLost", "Bytes Lost : {0} ({1}%) "), FText::AsNumber(Stats.TotalBytesLost), FText::AsNumber(PercentLoss, &Options));
}

#undef LOCTEXT_NAMESPACE
