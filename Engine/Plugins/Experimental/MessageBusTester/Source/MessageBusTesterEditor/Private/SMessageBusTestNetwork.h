// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SpscQueue.h"

#include "INetworkMessagingExtension.h"
#include "Framework/Docking/TabManager.h"

#include "Misc/DateTime.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SDiscoveredTesterListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SCompoundWidget.h"

class FWorkspaceItem;

struct FSegmenterRowData
{
	FSegmenterRowData(const FOutboundTransferStatistics& InStats)
		: Stats(InStats)
	{
		UpdateTime = AddTime = FDateTime::UtcNow();
	}

	bool operator < (const FSegmenterRowData &Other) const
	{
		// We want the messages to be in reverse order so we reverse the logic.
		return Stats.MessageId >= Other.Stats.MessageId;
	}

	bool operator == (const FOutboundTransferStatistics& Other) const
	{
		return Stats.MessageId == Other.MessageId;
	}

	FSegmenterRowData& operator=(const FOutboundTransferStatistics& InStats)
	{
		Stats = InStats;
		UpdateTime = FDateTime::UtcNow();
		return *this;
	}

	double ComputeTransferRate() const
	{
		FTimespan Span = UpdateTime - AddTime;
		double Seconds = Span.GetTotalSeconds();
		if (Seconds > UE_KINDA_SMALL_NUMBER )
		{
			// Compute the number of bits in our segment size.
			//
			constexpr double Bits = 8.0;
			constexpr double MegaBits = 1000000;
			return (double)(Stats.BytesSent * 8.0) / MegaBits /(double)Seconds;
		}
		return 0;
	}
	FDateTime		   AddTime;
	FDateTime		   UpdateTime;
	FOutboundTransferStatistics Stats;
};

class SMessageBusTesterSegmentersTableRow : public SMultiColumnTableRow<TSharedRef<FSegmenterRowData>>
{
	using Super = SMultiColumnTableRow<TSharedRef<FSegmenterRowData>>;

public:
	SLATE_BEGIN_ARGS(SMessageBusTesterSegmentersTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<FSegmenterRowData>, Segmenter)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwerTableView);


private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FText GetMessageId() const;
	FText GetSentSegments() const;
	FText GetAckSegments() const;
	FText GetSize() const;
	FText GetDataRate() const;

	TWeakPtr<FSegmenterRowData> Segmenter;
};

/**
 * Panel used to show stage monitoring data
 */
class SMessageBusTestNetwork : public SCompoundWidget
{
public:
	virtual ~SMessageBusTestNetwork();

private:
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SMessageBusTestNetwork) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FText GetIPAddress() const;
	FText GetAverageRTT() const;
	FText GetWindowSize() const;
	FText GetSegmentsInFlight() const;
	FText GetBytesSent() const;
	FText GetSegmentsLost() const;

	FMessageTransportStatistics GetStats() const;
	void SetNetworkViewData(TWeakPtr<FDiscoveredTesterTableRowData> View);

	using FSegmenterStatsArray = TArray<TSharedPtr<FSegmenterRowData>>;
	const FSegmenterStatsArray* GetSegmenterListSource() const;

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

private:
	void HandleSegmenterUpdated(FOutboundTransferStatistics InStats);

	TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FSegmenterRowData> InLogEntry, const TSharedRef<STableViewBase>& OwnerTable);

	TWeakPtr<FDiscoveredTesterTableRowData>   Item;

	TSharedPtr<SListView<TSharedPtr<FSegmenterRowData>>> SegmenterListView;

	TMap<FGuid, FSegmenterStatsArray>	AllSegmenters;

	TSpscQueue<FOutboundTransferStatistics> OutboundStats;
};
