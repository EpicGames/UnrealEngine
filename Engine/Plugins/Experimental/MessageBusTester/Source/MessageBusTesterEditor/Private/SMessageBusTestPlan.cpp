// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMessageBusTestPlan.h"

#include "Framework/Application/SlateApplication.h"

#include "SlateOptMacros.h"
#include "SMessageBusTestNetwork.h"

#include "Styling/SlateStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "MessageBusTesterEditorModule.h"
#include "IMessageBusTester.h"
#include "Widgets/SMessageBusTestPlanListView.h"
#include "Widgets/SAddTestPlanEntryDialog.h"
#include "DiscoveredTester.h"

#define LOCTEXT_NAMESPACE "SMessageBusTestPlan"



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMessageBusTestPlan::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Fill)
		[
			// Test plan entries
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				SNew(SMessageBusTestPlanListView)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ContentPadding(FMargin(4.f, 2.f))
				.OnClicked(this, &SMessageBusTestPlan::OnAddPayloadClicked)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddPayload", "Add"))
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ContentPadding(FMargin(4.f, 2.f))
				.OnClicked(this, &SMessageBusTestPlan::OnStartTestClicked)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StartActivity", "Start"))
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ContentPadding(FMargin(4.f, 2.f))
				.OnClicked(this, &SMessageBusTestPlan::OnStopTestClicked)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StopActivity", "Stop"))
				]
			]
		]
	];
}

FReply SMessageBusTestPlan::OnStartTestClicked()
{
	MessageBusTesterHelper::Get().GetMessageBusTester().StartTest();


	return FReply::Handled();
}


FReply SMessageBusTestPlan::OnStopTestClicked()
{
	MessageBusTesterHelper::Get().GetMessageBusTester().StopTest();
	return FReply::Handled();
}

FReply SMessageBusTestPlan::OnAddPayloadClicked()
{
	SAddTestPlanEntryDialog::OpenDialog();
	return FReply::Handled();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE





