// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/TabManager.h"

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "MessageBusTesterCommon.h"

class FWorkspaceItem;

/**
 * 
 */
class SMessageBusTestPlan : public SCompoundWidget
{
public:
	virtual ~SMessageBusTestPlan() = default;

private:
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SMessageBusTestPlan) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	
	FReply OnStartTestClicked();
	FReply OnStopTestClicked();
	FReply OnAddPayloadClicked();
};
