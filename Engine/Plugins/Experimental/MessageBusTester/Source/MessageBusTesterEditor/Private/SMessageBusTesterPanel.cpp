// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMessageBusTesterPanel.h"

#include "EditorFontGlyphs.h"
#include "EditorStyleSet.h"
#include "Styling/AppStyle.h"

#include "Math/Color.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IMessageBusTester.h"
#include "MessageBusTesterEditorModule.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SDiscoveredTesterListView.h"
#include "Widgets/SMessageBusTestPlanListView.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "SMessageBusTestPlan.h"
#include "SMessageBusTestLogger.h"

#define LOCTEXT_NAMESPACE "SMessageBusTesterPanel"

TWeakPtr<SMessageBusTesterPanel> SMessageBusTesterPanel::PanelInstance;
FDelegateHandle SMessageBusTesterPanel::LevelEditorTabManagerChangedHandle;


namespace MessageBusTesterUtilities
{
	static const FName NAME_App = FName("MessageBusTesterPanelApp");
	static const FName NAME_LevelEditorModuleName = FName("LevelEditor");

	TSharedRef<SDockTab> CreateApp(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SMessageBusTesterPanel)
			];
	}
}

FName SMessageBusTesterPanel::GetTabName()
{
	return MessageBusTesterUtilities::NAME_App;
}

void SMessageBusTesterPanel::RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MessageBusTesterUtilities::NAME_App, FOnSpawnTab::CreateStatic(&MessageBusTesterUtilities::CreateApp))
		.SetDisplayName(LOCTEXT("TabTitle", "MessageBus Tester"))
		.SetTooltipText(LOCTEXT("TooltipText", "Test MessageBus performance"))
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.GameSettings.Small")); 

#if WITH_EDITOR
	TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsMiscCategory());
#else
	TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());
#endif
}

void SMessageBusTesterPanel::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MessageBusTesterUtilities::NAME_App);
	}
}

TSharedPtr<SMessageBusTesterPanel> SMessageBusTesterPanel::GetPanelInstance()
{
	return SMessageBusTesterPanel::PanelInstance.Pin();
}

SMessageBusTesterPanel::~SMessageBusTesterPanel()
{
	if (DiscoveredTestersList.IsValid())
	{
		DiscoveredTestersList->OnSelectionChanged().RemoveAll(this);
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMessageBusTesterPanel::Construct(const FArguments& InArgs)
{
	PanelInstance = StaticCastSharedRef<SMessageBusTesterPanel>(AsShared());

	SAssignNew(DiscoveredTestersList, SDiscoveredTesterListView);

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SVerticalBox)
			// Toolbar
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 5.f, 5.f, 5.f)
			[
				MakeToolbarWidget()
			]
			+ SVerticalBox::Slot()
			.Padding(5.f, 5.f, 5.f, 5.f)
			.FillHeight(.3)
			[
				SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				+ SSplitter::Slot()
				.Value(.75f)
				[
					// Discovered tester List
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
					[
						DiscoveredTestersList.ToSharedRef()
					]
				]
				+ SSplitter::Slot()
				.Value(.25f)
				[
					// Data Provider Activities
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
					 [
						 SAssignNew(NetworkView, SMessageBusTestNetwork)
					 ]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(.3f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(.75f)
				[
					SAssignNew(Logger, SMessageBusTestLogger)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(.25f)
				[
					SAssignNew(TestPlan, SMessageBusTestPlan)
				]
			]
			// This tester status
			+ SVerticalBox::Slot()
			.Padding(5.f, 5.f, 5.f, 5.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
				[
					SNew(SHorizontalBox)
					// Spacer
					+ SHorizontalBox::Slot()
					[
						SNew(SSpacer)
					]
					// Status
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(10.f, 10.f, 10.f, 10.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TesterStatusLabel", "State : "))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(10.f, 10.f, 10.f, 10.f)
					[
						SNew(STextBlock)
						.Text(this, &SMessageBusTesterPanel::GetTesterStatus)
					]
				]
			]
		]
	];

	DiscoveredTestersList->OnSelectionChanged().AddRaw(this, &SMessageBusTesterPanel::HandleSelectionChange);
}

void SMessageBusTesterPanel::HandleSelectionChange(FDiscoveredTesterTableRowDataPtr RowData)
{
	if (NetworkView.IsValid())
	{
		NetworkView->SetNetworkViewData(RowData);
	}
}

TSharedRef<SWidget> SMessageBusTesterPanel::MakeToolbarWidget()
{
	TSharedRef<SWidget> Toolbar = SNew(SBorder)
	.VAlign(VAlign_Center)
	.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
	[
		SNew(SHorizontalBox)
		// Clear entries buttons
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.ToolTipText_Lambda([this]() {return GetStartStopButtonTooltip();})
				.ContentPadding(FMargin(4.f, 4.f))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SMessageBusTesterPanel::OnStartStopClicked)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Stop)
					.ColorAndOpacity_Lambda([this]() {return GetStartStopButtonColor();})
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.ToolTipText(LOCTEXT("ClearButton_ToolTip", "Clear lost providers."))
				.ContentPadding(FMargin(4.f, 4.f))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SMessageBusTesterPanel::OnClearClicked)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Eraser)
					.ColorAndOpacity(FLinearColor::White)
				]
			]
		]
	];

	return Toolbar;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
FText SMessageBusTesterPanel::GetStartStopButtonTooltip()
{
	if (MessageBusTesterHelper::Get().GetMessageBusTester().IsRunning())
	{
		return LOCTEXT("StopTesting", "Stop testing framework.");
	}
	else
	{
		return LOCTEXT("StartTesting", "Start testing framework.");
	}
}

FLinearColor SMessageBusTesterPanel::GetStartStopButtonColor()
{
	if (MessageBusTesterHelper::Get().GetMessageBusTester().IsRunning())
	{
		return FLinearColor::Green;
	}
	else
	{
		return FLinearColor::Red;
	}
}

FReply SMessageBusTesterPanel::OnStartStopClicked()
{
	if (!MessageBusTesterHelper::Get().GetMessageBusTester().IsRunning())
	{
		MessageBusTesterHelper::Get().GetMessageBusTester().StartSystem();
	}
	else
	{
		MessageBusTesterHelper::Get().GetMessageBusTester().StopSystem();
	}
	return FReply::Handled();
}

FReply SMessageBusTesterPanel::OnClearClicked()
{
	if (MessageBusTesterHelper::Get().GetMessageBusTester().ClearLostTesters())
	{
		DiscoveredTestersList->RebuildDiscoveredTesterList();		
	}
	return FReply::Handled();
}

FText SMessageBusTesterPanel::GetTesterStatus() const
{
	const EMessageBusTesterState State = MessageBusTesterHelper::Get().GetMessageBusTester().GetState();
	if (State == EMessageBusTesterState::Active)
	{
		return LOCTEXT("MonitorStatusActive", "Active");
	}
	else
	{
		return LOCTEXT("MonitorStatusInactive", "Inactive");
	}
}


#undef LOCTEXT_NAMESPACE
