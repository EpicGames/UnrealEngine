// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/TabManager.h"
#include "SMessageBusTestNetwork.h"

class SMessageBusTestPlan;
class SDiscoveredTesterListView;
class FWorkspaceItem;
class SMessageBusTestLogger;
struct FLinearColor;

/**
 * Main panel for Message Bus tester.
 */
class SMessageBusTesterPanel : public SCompoundWidget
{
public:
	virtual ~SMessageBusTesterPanel();

	static FName GetTabName();
	static void RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner();
	static TSharedPtr<SMessageBusTesterPanel> GetPanelInstance();

private:
	using Super = SCompoundWidget;
	static TWeakPtr<SMessageBusTesterPanel> PanelInstance;
	static FDelegateHandle LevelEditorTabManagerChangedHandle;

public:
	SLATE_BEGIN_ARGS(SMessageBusTesterPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void HandleSelectionChange(FDiscoveredTesterTableRowDataPtr RowData);
private:

	/** Make the toolbar widgets */
	TSharedRef<SWidget> MakeToolbarWidget();

	/** Get the tooltip text for the stop / start toolbar button. */
	FText GetStartStopButtonTooltip();

	/** Get the start stop button color. */
	FLinearColor GetStartStopButtonColor();
		
	/** Handles start / stop button clicked */
	FReply OnStartStopClicked();
		
	/** Handles clear everything button clicked */
	FReply OnClearClicked();

	/** Handles message bus tester settings button clicked */
	FReply OnShowProjectSettingsClicked();

	/** Returns tester status */
	FText GetTesterStatus() const;

	FReply OnStartTestClicked();
	FReply OnStopTestClicked();
	FReply OnAddPayloadClicked();

private:

	TSharedPtr<SMessageBusTestNetwork>    NetworkView;
	TSharedPtr<SDiscoveredTesterListView> DiscoveredTestersList;
	TSharedPtr<SMessageBusTestPlan> TestPlan;
	TSharedPtr<SMessageBusTestLogger> Logger;
};
