// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAddTestPlanEntryDialog.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "IMessageBusTester.h"
#include "MessageBusTesterEditorModule.h"

#include "Math/UnrealMath.h"

#include "EditorStyleSet.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AddTestPlanEntryDialog"

TWeakPtr<SWindow> SAddTestPlanEntryDialog::AddPointDialogWindow;

void SAddTestPlanEntryDialog::Construct(const FArguments& InArgs)
{
	const int32 MaxTestPlanBytes = 2 << 29;
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(0, 1, 0, 1))
					.FillWidth(1)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PayloadSizeLabel", "Payload size :"))
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SNumericEntryBox<int32>)
						.AllowSpin(false)
						.Value_Lambda([this]() { return TestPlanItem.NumBytes; })
						.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda([this, MaxTestPlanBytes](int32 NewValue) 
							{ TestPlanItem.NumBytes = FMath::Clamp<int32>(NewValue, 16, MaxTestPlanBytes); }))
						.OnValueCommitted(SNumericEntryBox<int32>::FOnValueCommitted::CreateLambda([this, MaxTestPlanBytes](int32 NewValue, ETextCommit::Type CommitType) 
							{ TestPlanItem.NumBytes = FMath::Clamp<int32>(NewValue, 16, MaxTestPlanBytes); }))
					]
				]
				+ SVerticalBox::Slot()
				.Padding(5.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(0, 1, 0, 1))
					.FillWidth(1)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("IntervalLabel", "Interval (s) :"))
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SNumericEntryBox<float>)
						.AllowSpin(false)
						.Value_Lambda([this]() { return TestPlanItem.IntervalSeconds; })
						.OnValueChanged(SNumericEntryBox<float>::FOnValueChanged::CreateLambda([this](float NewValue) { TestPlanItem.IntervalSeconds = FMath::Max(0.0,NewValue); }))
						.OnValueCommitted(SNumericEntryBox<float>::FOnValueCommitted::CreateLambda([this](float NewValue, ETextCommit::Type CommitType) { TestPlanItem.IntervalSeconds = FMath::Max(0.0,NewValue); }))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				MakeButtonsWidget()
			]
		]
	];
}

void SAddTestPlanEntryDialog::OpenDialog()
{
	TSharedPtr<SWindow> PopupWindowPin = AddPointDialogWindow.Pin();
	if (PopupWindowPin.IsValid())
	{
		PopupWindowPin->BringToFront();
	}
	else
	{
		PopupWindowPin = SNew(SWindow)
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.ClientSize(FVector2D(240, 180));

		FSlateApplication::Get().AddWindow(PopupWindowPin.ToSharedRef());
	}

	PopupWindowPin->SetTitle(LOCTEXT("AddTestPlanEntryDialog", "Add TestPlan item"));

	TSharedPtr<SAddTestPlanEntryDialog> NewDialogContent = SNew(SAddTestPlanEntryDialog);
	PopupWindowPin->SetContent(NewDialogContent.ToSharedRef());
	AddPointDialogWindow = PopupWindowPin;
}

FReply SAddTestPlanEntryDialog::OnAddEntryClicked()
{
	MessageBusTesterHelper::Get().GetMessageBusTester().AddTestPlanItem(TestPlanItem);
	CloseDialog();
	return FReply::Handled();
}

FReply SAddTestPlanEntryDialog::OnCancelClicked()
{
	CloseDialog();
	return FReply::Handled();
}

TSharedRef<SWidget> SAddTestPlanEntryDialog::MakeButtonsWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.OnClicked(this, &SAddTestPlanEntryDialog::OnAddEntryClicked)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("AddDataPoint", "Add"))
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.OnClicked(this, &SAddTestPlanEntryDialog::OnCancelClicked)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("CancelAddingDataPoint", "Cancel"))
		];
}

void SAddTestPlanEntryDialog::CloseDialog()
{
	TSharedPtr<SWindow> ExistingWindowPin = AddPointDialogWindow.Pin();
	if (ExistingWindowPin.IsValid())
	{
		ExistingWindowPin->RequestDestroyWindow();
		ExistingWindowPin = nullptr;
	}
}


#undef LOCTEXT_NAMESPACE 
