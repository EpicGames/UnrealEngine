// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "MessageBusTesterCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SAddTestPlanEntryDialog : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:

	SLATE_BEGIN_ARGS(SAddTestPlanEntryDialog)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	static void OpenDialog();

private:

	/** Called when Add button is clicked */
	FReply OnAddEntryClicked();
	
	/** Called when Cancel button is clicked */
	FReply OnCancelClicked();

	TSharedRef<SWidget> MakeButtonsWidget();

	/** Closes this dialog */
	void CloseDialog();
	
	void AddEntry() const;

private:
	FTestPlanItem TestPlanItem;

	static TWeakPtr<SWindow> AddPointDialogWindow;
};