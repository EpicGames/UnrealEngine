// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDiscoveredTesterListView.h"

#include "MessageBusTesterEditorModule.h"

#include "EditorFontGlyphs.h"
#include "EditorStyleSet.h"

#include "Framework/Views/ITypedTableView.h"
#include "IMessageBridge.h"
#include "IMessageBusTester.h"
#include "IMessageContext.h"

#include "INetworkMessagingExtension.h"

#include "Misc/App.h"
#include "MessageBusTesterCommon.h"
#include "MessageBusTesterEditorModule.h"
#include "DiscoveredTester.h"

#define LOCTEXT_NAMESPACE "SDiscoveredTesterListView"

namespace DiscoveredTesterListView
{
	const FName HeaderIdName_ConnectionState = TEXT("ConnectionState");
	const FName HeaderIdName_TestingState = TEXT("TestingState");
	const FName HeaderIdName_MachineName = TEXT("MachineName");
	const FName HeaderIdName_NetStats = TEXT("NetStats");
	const FName HeaderIdName_ProcessId = TEXT("ProcessId");
	const FName HeaderIdName_TesterName = TEXT("FriendlyName");
	const FName HeaderIdName_ReliableKeepAlive = TEXT("ReliableKeepAlive");
	const FName HeaderIdName_UnreliableKeepAlive = TEXT("UnreliableKeepAlive");
}

FDiscoveredTesterTableRowData::FDiscoveredTesterTableRowData(
	const FGuid& InIdentifier, const FTesterInstanceDescriptor& InDescriptor, const FMessageAddress& InAddress)
	: Identifier(InIdentifier)
	, Descriptor(InDescriptor)
	, Address(InAddress)
{
	if (INetworkMessagingExtension* NetworkMessagingExtension = FMessageBusTesterEditorModule::GetMessagingStatistics())
	{
		EndpointIdentifier = NetworkMessagingExtension->GetNodeIdFromAddress(Address);
	}
}

INetworkMessagingExtension* GetMessagingStatistics()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (IsInGameThread())
	{
		if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
		{
			return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
		}
	}
	else
	{
		ModularFeatures.LockModularFeatureList();
		ON_SCOPE_EXIT
		{
			ModularFeatures.UnlockModularFeatureList();
		};
		
		if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
		{
			return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
		}
	}
	
	ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
	return nullptr;
}

void FDiscoveredTesterTableRowData::UpdateCachedValues()
{
	const TConstArrayView<TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>> DiscoveredTesters = MessageBusTesterHelper::Get().GetMessageBusTester().GetDiscoveredTesters();
	const FGuid CurrentIdentifier = Identifier;
	const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>* LinkedTesterPtr = DiscoveredTesters.FindByPredicate([CurrentIdentifier](const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Other) { return CurrentIdentifier == Other->Identifier; });
	if (LinkedTesterPtr)
	{
		const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe> LinkedTester = *LinkedTesterPtr;
		CachedTestingState = LinkedTester->State;
		CachedConnectionState = LinkedTester->ConnectionState;

		CachedKeepAliveStatistics = LinkedTester->KeepAliveStatistics;
	}
	else
	{
		CachedTestingState = EMessageBusTesterState::Idle;
		CachedConnectionState = EDiscoveredTesterConnectionState::Lost;
	}

	if (INetworkMessagingExtension* NetworkMessagingExtension = FMessageBusTesterEditorModule::GetMessagingStatistics())
	{
		// The endpoint identifer can change if the remote host "resets" UdpMessaging. We always get the latest endpoint identifier so that
		// we can properly track statistics. 
		EndpointIdentifier = NetworkMessagingExtension->GetNodeIdFromAddress(Address);
		Statistics = NetworkMessagingExtension->GetLatestNetworkStatistics(EndpointIdentifier);
	}
}

FText FDiscoveredTesterTableRowData::GetNetSummary()
{
	return FText::Format(LOCTEXT("NetStats", "{0} Snt / {1} Rcv / {2} Ack"),
						 FText::AsNumber(Statistics.PacketsSent),
						 FText::AsNumber(Statistics.PacketsReceived),
						 FText::AsNumber(Statistics.PacketsAcked));
}

/**
 * SDiscoveredTesterTableRow
 */
void SDiscoveredTesterTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
{
	Item = InArgs._Item;
	check(Item.IsValid());

	Super::FArguments Arg;
	Super::Construct(Arg, InOwerTableView);
}

TSharedRef<SWidget> SDiscoveredTesterTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (DiscoveredTesterListView::HeaderIdName_ConnectionState == ColumnName)
	{
		return SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
			.Text(this, &SDiscoveredTesterTableRow::GetConnectionStateGlyphs)
			.ColorAndOpacity(this, &SDiscoveredTesterTableRow::GetConnectionStateColorAndOpacity);
	}
	if (DiscoveredTesterListView::HeaderIdName_TestingState == ColumnName)
	{
		return SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
			.Text(this, &SDiscoveredTesterTableRow::GetTestingStateGlyphs)
			.ColorAndOpacity(this, &SDiscoveredTesterTableRow::GetTestingStateColorAndOpacity);
	}
	if (DiscoveredTesterListView::HeaderIdName_MachineName == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDiscoveredTesterTableRow::GetMachineName)
			];
	}
	if (DiscoveredTesterListView::HeaderIdName_TesterName == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDiscoveredTesterTableRow::GetFriendlyName)
			];
	}
	if (DiscoveredTesterListView::HeaderIdName_ProcessId == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDiscoveredTesterTableRow::GetProcessId)
			];
	}
	if (DiscoveredTesterListView::HeaderIdName_NetStats == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDiscoveredTesterTableRow::GetNetStats)
			];
	}
	if (DiscoveredTesterListView::HeaderIdName_ReliableKeepAlive == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDiscoveredTesterTableRow::GetReliableKeepAliveStats)
			];
	}
	if (DiscoveredTesterListView::HeaderIdName_UnreliableKeepAlive == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDiscoveredTesterTableRow::GetUnreliableKeepAliveStats)
			];
	}
	
	return SNullWidget::NullWidget;
}

FText SDiscoveredTesterTableRow::GetConnectionStateGlyphs() const
{
	return FEditorFontGlyphs::Circle;
}

FSlateColor SDiscoveredTesterTableRow::GetConnectionStateColorAndOpacity() const
{
	switch (Item->CachedConnectionState)
	{
		case EDiscoveredTesterConnectionState::Connected:
		{
			return FLinearColor::Green;
		}
		case EDiscoveredTesterConnectionState::Lost:
		default:
		{
			return FLinearColor::Red;
		}
	}
}

FText SDiscoveredTesterTableRow::GetTestingStateGlyphs() const
{
	return FEditorFontGlyphs::Long_Arrow_Down;
}

FSlateColor SDiscoveredTesterTableRow::GetTestingStateColorAndOpacity() const
{
	switch (Item->CachedTestingState)
	{
	case EMessageBusTesterState::Active:
	{
		return FLinearColor::Green;
	}
	case EMessageBusTesterState::Idle:
	default:
	{
		return FLinearColor::White;
	}
	}
}

FText SDiscoveredTesterTableRow::GetMachineName() const
{
	return FText::FromString(Item->Descriptor.MachineName);
}

FText SDiscoveredTesterTableRow::GetProcessId() const
{
	return FText::AsNumber(Item->Descriptor.ProcessId);
}

FText SDiscoveredTesterTableRow::GetNetStats() const
{
	return Item->GetNetSummary();
}

FText SDiscoveredTesterTableRow::GetReliableKeepAliveStats() const
{
	return FText::Format(LOCTEXT("ReliableKeepAliveStats", "{0} [{1} / {2} / {3}]"),
		FText::AsNumber(Item->CachedKeepAliveStatistics.LastReliableKeepAliveReceptionInterval),
		FText::AsNumber(Item->CachedKeepAliveStatistics.MinReliableKeepAliveInterval),
		FText::AsNumber(Item->CachedKeepAliveStatistics.AverageReliableKeepAliveInterval),
		FText::AsNumber(Item->CachedKeepAliveStatistics.MaxReliableKeepAliveInterval));
}

FText SDiscoveredTesterTableRow::GetUnreliableKeepAliveStats() const
{
	return FText::Format(LOCTEXT("UnreliableKeepAliveStats", "{0} [{1} / {2} / {3}]"),
		FText::AsNumber(Item->CachedKeepAliveStatistics.LastUnreliableKeepAliveReceptionInterval),
		FText::AsNumber(Item->CachedKeepAliveStatistics.MinUnreliableKeepAliveInterval),
		FText::AsNumber(Item->CachedKeepAliveStatistics.AverageUnreliableKeepAliveInterval),
		FText::AsNumber(Item->CachedKeepAliveStatistics.MaxUnreliableKeepAliveInterval));
}

FText SDiscoveredTesterTableRow::GetFriendlyName() const
{
	return FText::FromName(Item->Descriptor.FriendlyName);
}

void SDiscoveredTesterListView::HandleSelectionChanged(FDiscoveredTesterTableRowDataPtr Item, ESelectInfo::Type /*SelectInfo*/ )
{
	OnSelectionChanged().Broadcast(Item);
}

/**
 * SDiscoveredTesterListView
 */
void SDiscoveredTesterListView::Construct(const FArguments& InArgs)
{
	MessageBusTesterHelper::Get().GetMessageBusTester().OnDiscoveredTesterListChanged().AddSP(this, &SDiscoveredTesterListView::OnDiscoveredTesterListChanged);

	Super::Construct
	(
		Super::FArguments()
		.ListItemsSource(&ListItemsSource)
		.SelectionMode(ESelectionMode::Single)
		.OnSelectionChanged(this, &SDiscoveredTesterListView::HandleSelectionChanged)
		.OnGenerateRow(this, &SDiscoveredTesterListView::OnGenerateRow)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true) //To show/hide columns

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_ConnectionState)
			.FixedWidth(30.f)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Center)
			.DefaultLabel(LOCTEXT("HeaderName_ConnectionState", "ConnectionState"))

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_TestingState)
			.FixedWidth(30.f)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Center)
			.DefaultLabel(LOCTEXT("HeaderName_TestingState", "Testing"))

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_MachineName)
			.FillWidth(.1f)
			.DefaultLabel(LOCTEXT("HeaderName_MachineName", "Machine"))

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_ProcessId)
			.FillWidth(.1f)
			.DefaultLabel(LOCTEXT("HeaderName_ProcessId", "Process Id"))

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_TesterName)
			.FillWidth(.15f)
			.DefaultLabel(LOCTEXT("HeaderName_FriendlyName", "Friendly Name"))
			.ShouldGenerateWidget(true) //Can't hide this column

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_UnreliableKeepAlive)
			.FillWidth(.2f)
			.DefaultLabel(LOCTEXT("HeaderName_UnreliableKeepAlive", "Unreliable KeepAlive"))

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_ReliableKeepAlive)
			.FillWidth(.2f)
			.DefaultLabel(LOCTEXT("HeaderName_ReliableKeepAlive", "Reliable KeepAlive"))

			+ SHeaderRow::Column(DiscoveredTesterListView::HeaderIdName_NetStats)
			.FillWidth(.2f)
			.DefaultLabel(LOCTEXT("HeaderName_NetStats", "Net stats"))
		)
	);

	RebuildDiscoveredTesterList();
}

SDiscoveredTesterListView::~SDiscoveredTesterListView()
{
	if (MessageBusTesterHelper::IsAvailable())
	{
		MessageBusTesterHelper::Get().GetMessageBusTester().OnDiscoveredTesterListChanged().RemoveAll(this);
	}
}

void SDiscoveredTesterListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	bool bForceRefresh = false;
	if (bRebuildListRequested)
	{
		RebuildDiscoveredTesterList();
		RebuildList();
		bRebuildListRequested = false;
		bForceRefresh = true;
	}

	//Update cached values at specific rate or when list has been rebuilt
	constexpr double RefreshRate = .33;
	if (bForceRefresh || (InCurrentTime - LastRefreshTime > RefreshRate))
	{
		LastRefreshTime = InCurrentTime;
		for (FDiscoveredTesterTableRowDataPtr& RowDataPtr : ListItemsSource)
		{
			RowDataPtr->UpdateCachedValues();
		}
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedRef<ITableRow> SDiscoveredTesterListView::OnGenerateRow(FDiscoveredTesterTableRowDataPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SDiscoveredTesterTableRow> Row = SNew(SDiscoveredTesterTableRow, OwnerTable).Item(InItem);
	ListRowWidgets.Add(Row);
	return Row;
}

void SDiscoveredTesterListView::OnDiscoveredTesterListChanged()
{
	bRebuildListRequested = true;
}

void SDiscoveredTesterListView::RebuildDiscoveredTesterList()
{
	ListItemsSource.Reset();

	const TConstArrayView<TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>> DiscoveredTesters = MessageBusTesterHelper::Get().GetMessageBusTester().GetDiscoveredTesters();
	for (const TSharedPtr<FDiscoveredTester, ESPMode::ThreadSafe>& Tester : DiscoveredTesters)
	{
		TSharedRef<FDiscoveredTesterTableRowData> RowData = MakeShared<FDiscoveredTesterTableRowData>(Tester->Identifier, Tester->Descriptor, Tester->Address);
		ListItemsSource.Add(RowData);
	}

	for (FDiscoveredTesterTableRowDataPtr& TableRowData : ListItemsSource)
	{
		TableRowData->UpdateCachedValues();
	}

	RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE

