// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#include "CoreMinimal.h"
#include "DiscoveredTester.h"
#include "MessageBusTesterEditorModule.h"
#include "INetworkMessagingExtension.h"
#include "MessageBusTesterCommon.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"




class IMessageBusTester;
/**
 * FDiscoveredTesterTableRowData
 */
struct FDiscoveredTesterTableRowData : TSharedFromThis<FDiscoveredTesterTableRowData>
{
	FDiscoveredTesterTableRowData(const FGuid& InIdentifier, const FTesterInstanceDescriptor& InDescriptor, const FMessageAddress& InAddress);

	/** Fetch latest information for the associated data provider */
	void UpdateCachedValues();

	/** Get a summary of current network statistics. */
	FText GetNetSummary();

	/** Return the current network statistics. */
	const FMessageTransportStatistics& LatestStats()
	{
		return Statistics;
	}

public:
	FGuid Identifier;
	FTesterInstanceDescriptor Descriptor;

	FGuid EndpointIdentifier;
	FMessageAddress Address;

	FMessageTransportStatistics Statistics;
	EDiscoveredTesterConnectionState CachedConnectionState;
	EMessageBusTesterState CachedTestingState;
	FKeepAliveStatistics CachedKeepAliveStatistics;
};


using FDiscoveredTesterTableRowDataPtr = TSharedPtr<FDiscoveredTesterTableRowData>;


/**
 *
 */
class SDiscoveredTesterTableRow : public SMultiColumnTableRow<FDiscoveredTesterTableRowDataPtr>
{
	using Super = SMultiColumnTableRow<FDiscoveredTesterTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(SDiscoveredTesterTableRow) { }
		SLATE_ARGUMENT(FDiscoveredTesterTableRowDataPtr, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwerTableView);

private:

	/** Handles creation of each columns widget */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	/** Getters to populate the UI */
	FText GetConnectionStateGlyphs() const;
	FSlateColor GetConnectionStateColorAndOpacity() const;
	FText GetTestingStateGlyphs() const;
	FSlateColor GetTestingStateColorAndOpacity() const;
	FText GetMachineName() const;
	FText GetProcessId() const;
	FText GetFriendlyName() const;
	FText GetNetStats() const;
	FText GetReliableKeepAliveStats() const;
	FText GetUnreliableKeepAliveStats() const;

private:

	/** Item to display */
	FDiscoveredTesterTableRowDataPtr Item;

	/** Last time we refreshed the UI */
	double LastRefreshTime = 0.0;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChangedNetworkItem, FDiscoveredTesterTableRowDataPtr);

/**
 *
 */
class SDiscoveredTesterListView : public SListView<FDiscoveredTesterTableRowDataPtr>
{
	using Super = SListView<FDiscoveredTesterTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(SDiscoveredTesterListView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Cleanup ourselves */
	virtual ~SDiscoveredTesterListView();

	/** Used to refresh cached values shown in UI */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Delegate that is called whenever the selection is changed on the network list.
	 */
	FOnSelectionChangedNetworkItem& OnSelectionChanged()
	{
		return SelectionChangeEvent;
	}

	/** Rebuild provider list from scratch */
	void RebuildDiscoveredTesterList();

private:

	/** Generate a new row for the listview using the Item data */
	TSharedRef<ITableRow> OnGenerateRow(FDiscoveredTesterTableRowDataPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for when the list view selection changes. */
	void HandleSelectionChanged(FDiscoveredTesterTableRowDataPtr, ESelectInfo::Type /*SelectInfo*/ );

	void OnDiscoveredTesterListChanged();


private:

	TArray<FDiscoveredTesterTableRowDataPtr> ListItemsSource;
	TArray<TWeakPtr<SDiscoveredTesterTableRow>> ListRowWidgets;

	/** Used to cache if list needs to be refreshed or not */
	bool bRebuildListRequested = false;

	/** Timestamp when we last refreshed UI */
	double LastRefreshTime = 0.0;

	FOnSelectionChangedNetworkItem SelectionChangeEvent;
};

