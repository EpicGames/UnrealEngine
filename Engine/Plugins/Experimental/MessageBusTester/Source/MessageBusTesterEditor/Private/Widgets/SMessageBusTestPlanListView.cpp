// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMessageBusTestPlanListView.h"

#include "EditorStyleSet.h"
#include "Styling/AppStyle.h"

#include "IMessageBusTester.h"
#include "Misc/App.h"
#include "MessageBusTesterCommon.h"
#include "MessageBusTesterEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SMessageBusTestPlanListView"

namespace MessageBusTestPlanListView
{
	const FName HeaderIdName_PayloadSize = TEXT("PayloadSize");
	const FName HeaderIdName_Interval = TEXT("Interval");
	const FName HeaderIdName_Remove = TEXT("Remove");
}

/**
 * FMessageBusTestPlanTableRowData
 */
struct FMessageBusTestPlanTableRowData : TSharedFromThis<FMessageBusTestPlanTableRowData>
{
	FMessageBusTestPlanTableRowData(int32 InIndex, int32 InPayloadSizeBytes, double InIntervalSeconds)
		: EntryIndex(InIndex)
		, PayloadSizeBytes(InPayloadSizeBytes)
		, IntervalSeconds(InIntervalSeconds)
	{
	}

public:
	
	int32 EntryIndex = INDEX_NONE;
	int32 PayloadSizeBytes = 0;
	double IntervalSeconds = 0.0;
};


/**
 * SMessageBusTestPlanTableRow
 */
void SMessageBusTestPlanTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
{
	Item = InArgs._Item;
	check(Item.IsValid());

	Super::FArguments Arg;
	Super::Construct(Arg, InOwerTableView);
}

TSharedRef<SWidget> SMessageBusTestPlanTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (MessageBusTestPlanListView::HeaderIdName_PayloadSize == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestPlanTableRow::GetPayloadSize)
			];
	}
	if (MessageBusTestPlanListView::HeaderIdName_Interval == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SMessageBusTestPlanTableRow::GetInterval)
			];
	}
	if (MessageBusTestPlanListView::HeaderIdName_Remove == ColumnName)
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SMessageBusTestPlanTableRow::OnRemoveEntryClicked)
			.ToolTipText(LOCTEXT("RemoveTestPlanEntry", "Remove entry from test plan"))
			.ContentPadding(0.f)
			.ForegroundColor(FSlateColor::UseForeground())
			.IsFocusable(false)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.Button_EmptyArray"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}
	
	return SNullWidget::NullWidget;
}

FReply SMessageBusTestPlanTableRow::OnRemoveEntryClicked()
{
	MessageBusTesterHelper::Get().GetMessageBusTester().RemoveTestPlanItem(Item->EntryIndex);
	return FReply::Handled();
}

FText SMessageBusTestPlanTableRow::GetPayloadSize() const
{
	return FText::AsNumber(Item->PayloadSizeBytes);
}

FText SMessageBusTestPlanTableRow::GetInterval() const
{
	return FText::AsNumber(Item->IntervalSeconds);
}

SMessageBusTestPlanListView::~SMessageBusTestPlanListView()
{
	if (MessageBusTesterHelper::IsAvailable())
	{
		MessageBusTesterHelper::Get().GetMessageBusTester().OnTestPlanChanged().RemoveAll(this);
	}
}

/**
 * SMessageBusTestPlanListView
 */
void SMessageBusTestPlanListView::Construct(const FArguments& InArgs)
{
	MessageBusTesterHelper::Get().GetMessageBusTester().OnTestPlanChanged().AddSP(this, &SMessageBusTestPlanListView::OnMessageBusTestPlanChanged);

	Super::Construct
	(
		Super::FArguments()
		.ListItemsSource(&ListItemsSource)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SMessageBusTestPlanListView::OnGenerateRow)
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column(MessageBusTestPlanListView::HeaderIdName_PayloadSize)
			.FillWidth(.6f)
			.DefaultLabel(LOCTEXT("HeaderName_Payload", "Size (bytes)"))

			+ SHeaderRow::Column(MessageBusTestPlanListView::HeaderIdName_Interval)
			.FillWidth(.4f)
			.DefaultLabel(LOCTEXT("HeaderName_Interval", "Interval (s)"))

			+ SHeaderRow::Column(MessageBusTestPlanListView::HeaderIdName_Remove)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Center)
			.ManualWidth(20.f)
			.DefaultLabel(FText::GetEmpty())
		)
	);

	RebuildMessageBusTestPlanList();
}

void SMessageBusTestPlanListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRebuildListRequested)
	{
		RebuildMessageBusTestPlanList();
		RebuildList();
		bRebuildListRequested = false;
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedRef<ITableRow> SMessageBusTestPlanListView::OnGenerateRow(FMessageBusTestPlanTableRowDataPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SMessageBusTestPlanTableRow> Row = SNew(SMessageBusTestPlanTableRow, OwnerTable)
		.Item(InItem);
	return Row;
}

void SMessageBusTestPlanListView::OnMessageBusTestPlanChanged()
{
	bRebuildListRequested = true;
}

void SMessageBusTestPlanListView::RebuildMessageBusTestPlanList()
{
	ListItemsSource.Reset();

	const FMessageBusTestPlan& MessageBusTestPlan = MessageBusTesterHelper::Get().GetMessageBusTester().GetTestPlan();
	for (int32 Index = 0; Index < MessageBusTestPlan.TestPlanItems.Num(); ++Index)
	{
		TSharedRef<FMessageBusTestPlanTableRowData> RowData = MakeShared<FMessageBusTestPlanTableRowData>(Index, MessageBusTestPlan.TestPlanItems[Index].NumBytes, MessageBusTestPlan.TestPlanItems[Index].IntervalSeconds);
		ListItemsSource.Add(RowData);
	}

	RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE

