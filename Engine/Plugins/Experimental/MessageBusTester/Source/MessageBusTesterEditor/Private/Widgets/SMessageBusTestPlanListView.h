// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

struct FMessageBusTestPlanTableRowData;
class IMessageBusTester;


using FMessageBusTestPlanTableRowDataPtr = TSharedPtr<FMessageBusTestPlanTableRowData>;


/**
 *	
 */
class SMessageBusTestPlanTableRow : public SMultiColumnTableRow<FMessageBusTestPlanTableRowDataPtr>
{
	using Super = SMultiColumnTableRow<FMessageBusTestPlanTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(SMessageBusTestPlanTableRow) { }
		SLATE_ARGUMENT(FMessageBusTestPlanTableRowDataPtr, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwerTableView);

private:

	/** Handles creation of each columns widget */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FReply OnRemoveEntryClicked();

	/** Getters to populate the UI */
	FText GetPayloadSize() const;
	FText GetInterval() const;

private:

	/** Item to display */
	FMessageBusTestPlanTableRowDataPtr Item;
	
	/** Last time we refreshed the UI */
	double LastRefreshTime = 0.0;
};


/**
 *
 */
class SMessageBusTestPlanListView : public SListView<FMessageBusTestPlanTableRowDataPtr>
{
	using Super = SListView<FMessageBusTestPlanTableRowDataPtr>;

public:
	virtual ~SMessageBusTestPlanListView();
		
	SLATE_BEGIN_ARGS(SMessageBusTestPlanListView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Used to refresh cached values shown in UI */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/** Generate a new row for the listview using the Item data */
	TSharedRef<ITableRow> OnGenerateRow(FMessageBusTestPlanTableRowDataPtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	
	void OnMessageBusTestPlanChanged();
	
	/** Rebuild list from scratch */
	void RebuildMessageBusTestPlanList();

private:
	
	TArray<FMessageBusTestPlanTableRowDataPtr> ListItemsSource;
	TArray<TWeakPtr<SMessageBusTestPlanTableRow>> ListRowWidgets;

	/** Used to cache if list needs to be refreshed or not */
	bool bRebuildListRequested = false;

	/** Timestamp when we last refreshed UI */
	double LastRefreshTime = 0.0;
};

