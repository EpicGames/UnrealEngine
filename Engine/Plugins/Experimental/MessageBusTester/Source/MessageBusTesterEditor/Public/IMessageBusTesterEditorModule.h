// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class IMessageBusTesterEditorModule : public IModuleInterface
{
public:
	virtual void DisplayMessageBusTester() = 0;
};

